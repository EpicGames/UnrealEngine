{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "OpenXR visionOS",
	"Description": "OpenXR visionOS exposes visionOS full immersion to Unreal as if it was an OpenXR runtime.  Documentation on developing VR in Unreal with OpenXR is applicable.",
	"Category": "Virtual Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"VisionOS"
	],
	"Modules": [
		{
			"Name": "OXRVisionOS",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"VisionOS"
			]
		},
		{
			"Name": "OXRVisionOSSettings",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"VisionOS",
				"Win64",
				"Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OpenXR",
			"Enabled": true
		}
	]
}
