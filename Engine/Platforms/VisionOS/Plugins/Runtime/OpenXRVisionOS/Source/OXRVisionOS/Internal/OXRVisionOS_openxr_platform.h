// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "openxr/openxr.h"
#include "RHIFwd.h"

// Uncomment to make intellisense work better.
//#include "../../../../../../../../Source/ThirdParty/OpenXR/include/openxr/openxr.h"

// This number is in a block that is unused by openxr, and that seemed unlikely to be used soon based on the patterns of usage.
// But if it ever was used by openxr we would have to change it.
#define XR_TYPE_OXRVISIONOS_EPIC 1000059100

typedef enum XrOXRVisionOSStructureTypeEPIC {
	XR_TYPE_GRAPHICS_BINDING_OXRVISIONOS_EPIC = XR_TYPE_OXRVISIONOS_EPIC + 0,
	XR_TYPE_SWAPCHAIN_IMAGE_OXRVISIONOS_EPIC = XR_TYPE_OXRVISIONOS_EPIC + 1,
} XrOXRVisionOSStructureTypeEPIC;

typedef struct XrGraphicsBindingOXRVisionOSEPIC {
	XrStructureType             type;
	const void* XR_MAY_ALIAS    next;
	// More members here
} XrGraphicsBindingOXRVisionOSEPIC;                                                                                                                                                 

typedef struct XrSwapchainImageOXRVisionOS {
    XrStructureType       type;
    void* XR_MAY_ALIAS    next;
	FTextureRHIRef        image;
} XrSwapchainImageOXRVisionOS;


// OXRVisionOS Extensions

#define XR_EPIC_oxrvisionos_controller 1
#define XR_EPIC_oxrvisionos_controller_SPEC_VERSION 1
#define XR_EPIC_OXRVISIONOS_CONTROLLER_NAME "XR_EPIC_oxrvisionos_controller"