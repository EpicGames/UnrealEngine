// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class OXRVisionOS : ModuleRules
	{
        public OXRVisionOS(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"RHI",
				"MetalRHI",
				"OpenXR",
				"OpenXRHMD",
				"OXRVisionOSSettings",
			});
			
			PublicWeakFrameworks.Add("Metal");
			PublicWeakFrameworks.Add("ARKit");		
			PrivateIncludePathModuleNames.Add("OpenXRHMD");
		}
	}
}
