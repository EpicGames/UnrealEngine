// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOS.h"
#include "Engine/Engine.h"
#include "Modules/ModuleManager.h"

#include "OpenXRCore.h"
#include "OXRVisionOSInstance.h"
#include "OXRVisionOSSession.h"
#include "OXRVisionOSSpace.h"
#include "OXRVisionOSActionSet.h"
#include "OXRVisionOSAction.h"
#include "OXRVisionOS_RenderBridge.h"
#include "OXRVisionOSSwapchain.h"
#include "OXRVisionOSPlatformUtils.h"
#include "OXRVisionOS_openxr_platform.h"


DEFINE_LOG_CATEGORY(LogOXRVisionOS);

TSharedPtr<FOXRVisionOSInstance, ESPMode::ThreadSafe> FOXRVisionOS::Instance;

#define OXRVISIONOS_CHECK_MODULE_EARLY_RETURN(Module) \
check(Module); \
if (Module == nullptr) \
{ \
	return XrResult::XR_ERROR_RUNTIME_FAILURE; \
}

#define OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(Instance) \
if (!FOXRVisionOS::CheckInstance(Instance)) \
{ \
	return XrResult::XR_ERROR_HANDLE_INVALID; \
}

#define OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(Session) \
if (!FOXRVisionOSInstance::CheckSession(Session)) \
{ \
	return XrResult::XR_ERROR_HANDLE_INVALID; \
}

#define OXRVISIONOS_CHECK_SWAPCHAIN_EARLY_RETURN(SwapChain) \
if (SwapChain == XR_NULL_HANDLE) \
{ \
	return XrResult::XR_ERROR_HANDLE_INVALID; \
}

#if 0
#define OXRVISIONOS_LOG_RESULT(Result) \
{ \
	UE_LOGF(LogOXRVisionOS, Verbose, "%ls: %ls", ANSI_TO_TCHAR(__FUNCTION__), OpenXRResultToString(Result)); \
}
#else
#define OXRVISIONOS_LOG_RESULT(Result)
#endif


namespace OXRVisionOS
{
	TMap<FString, PFN_xrVoidFunction> GlobalFunctionMap;
	TMap<FString, PFN_xrVoidFunction> InstanceFunctionMap;

	XRAPI_ATTR XrResult XRAPI_CALL xrGetInstanceProcAddr(
		XrInstance                                  instance,
		const char*									name,
		PFN_xrVoidFunction*							function)
	{
		if (instance == XR_NULL_HANDLE)
		{
			PFN_xrVoidFunction* itr = GlobalFunctionMap.Find(name);
			if (itr)
			{
				*function = *itr;
				return XrResult::XR_SUCCESS;
			}
			else
			{
				return XrResult::XR_ERROR_FUNCTION_UNSUPPORTED;
			}

		}
		else
		{
			PFN_xrVoidFunction* itr = InstanceFunctionMap.Find(name);
			if (itr)
			{
				*function = *itr;
				return XrResult::XR_SUCCESS;
			}
			else
			{
				return XrResult::XR_ERROR_FUNCTION_UNSUPPORTED;
			}
		}
	}


	// Global Functions

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateApiLayerProperties(
		uint32_t                                    propertyCapacityInput,
		uint32_t*									propertyCountOutput,
		XrApiLayerProperties*						properties)
	{
		FOXRVisionOS* Module = FOXRVisionOS::Get();
		OXRVISIONOS_CHECK_MODULE_EARLY_RETURN(Module);

		return Module->XrEnumerateApiLayerProperties(
			propertyCapacityInput,
			propertyCountOutput,
			properties);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateInstanceExtensionProperties(
		const char*									layerName,
		uint32_t                                    propertyCapacityInput,
		uint32_t*									propertyCountOutput,
		XrExtensionProperties*						properties)
	{
		FOXRVisionOS* Module = FOXRVisionOS::Get();
		OXRVISIONOS_CHECK_MODULE_EARLY_RETURN(Module);

		return Module->XrEnumerateInstanceExtensionProperties(
			layerName,
			propertyCapacityInput,
			propertyCountOutput,
			properties);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrCreateInstance(
		const XrInstanceCreateInfo*					createInfo,
		XrInstance*									instance)
	{
		FOXRVisionOS* Module = FOXRVisionOS::Get();
		OXRVISIONOS_CHECK_MODULE_EARLY_RETURN(Module);

		XrResult Result = Module->XrCreateInstance(createInfo, instance);
		OXRVISIONOS_LOG_RESULT(Result);
		return Result;
	}


	// Instance functions

	XRAPI_ATTR XrResult XRAPI_CALL xrDestroyInstance(
		XrInstance                                  instance)
	{
		FOXRVisionOS* Module = FOXRVisionOS::Get();
		check(Module);
		if (Module == nullptr)
		{
			check(false);
			return XrResult::XR_ERROR_HANDLE_INVALID; // This is the only allowed failure code. Not really appropriate, but this should never happen.
		}

		XrResult Result = Module->XrDestroyInstance(instance);
		OXRVISIONOS_LOG_RESULT(Result);
		return Result;
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetInstanceProperties(
		XrInstance                                  instance,
		XrInstanceProperties* instanceProperties)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);

		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrGetInstanceProperties(instanceProperties);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrPollEvent(
		XrInstance                                  instance,
		XrEventDataBuffer*							eventData)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);

		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrPollEvent(eventData);
	}

#	define OXRVISIONOS_XR_RESULT_STR(name, val) case name: FCStringAnsi::Strncpy(buffer, #name, XR_MAX_RESULT_STRING_SIZE); return XrResult::XR_SUCCESS;

	XRAPI_ATTR XrResult XRAPI_CALL xrResultToString(
		XrInstance                                  instance,
		XrResult                                    value,
		char                                        buffer[XR_MAX_RESULT_STRING_SIZE])
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		switch (value)
		{
			XR_LIST_ENUM_XrResult(OXRVISIONOS_XR_RESULT_STR);
			default: 
			{
				if (value > 0)
				{
					FCStringAnsi::Snprintf(buffer, XR_MAX_RESULT_STRING_SIZE, "XR_UNKNOWN_SUCCESS_%i", int(value));
				}
				else
				{
					FCStringAnsi::Snprintf(buffer, XR_MAX_RESULT_STRING_SIZE, "XR_UNKNOWN_FAILURE_%i", int(value));
				}
				return XrResult::XR_SUCCESS;
			}
		}
	}

#	define OXRVISIONOS_XR_STRUCTURETYPE_STR(name, val) case name: FCStringAnsi::Strncpy(buffer, #name, XR_MAX_STRUCTURE_NAME_SIZE); return XrResult::XR_SUCCESS;

	XRAPI_ATTR XrResult XRAPI_CALL xrStructureTypeToString(
		XrInstance                                  instance,
		XrStructureType                             value,
		char                                        buffer[XR_MAX_STRUCTURE_NAME_SIZE])
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		switch (value)
		{
			XR_LIST_ENUM_XrStructureType(OXRVISIONOS_XR_STRUCTURETYPE_STR);
			default:
			{
				FCStringAnsi::Snprintf(buffer, XR_MAX_STRUCTURE_NAME_SIZE, "XR_UNKNOWN_STRUCTURE_TYPE_ %i", int(value));
				return XrResult::XR_SUCCESS;
			}
		}
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetSystem(
		XrInstance                                  instance,
		const XrSystemGetInfo*						getInfo,
		XrSystemId*									systemId)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;

		XrResult Result = Instance->XrGetSystem(getInfo, systemId);
		OXRVISIONOS_LOG_RESULT(Result);
		return Result;
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetSystemProperties(
		XrInstance                                  instance,
		XrSystemId                                  systemId,
		XrSystemProperties*							properties)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrGetSystemProperties(systemId, properties);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateEnvironmentBlendModes(
		XrInstance                                  instance,
		XrSystemId                                  systemId,
		XrViewConfigurationType                     viewConfigurationType,
		uint32_t                                    environmentBlendModeCapacityInput,
		uint32_t*									environmentBlendModeCountOutput,
		XrEnvironmentBlendMode*						environmentBlendModes)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrEnumerateEnvironmentBlendModes(	systemId,
															viewConfigurationType,
															environmentBlendModeCapacityInput,
															environmentBlendModeCountOutput,
															environmentBlendModes);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrCreateSession(
		XrInstance                                  instance,
		const XrSessionCreateInfo*					createInfo,
		XrSession*									session)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		XrResult Result = Instance->XrCreateSession(createInfo, session);
		OXRVISIONOS_LOG_RESULT(Result);
		return Result;
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrDestroySession(
		XrSession                                   session)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		XrResult Result = Session->XrDestroySession();
		OXRVISIONOS_LOG_RESULT(Result);
		return Result;
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateReferenceSpaces(
		XrSession                                   session,
		uint32_t                                    spaceCapacityInput,
		uint32_t*									spaceCountOutput,
		XrReferenceSpaceType*						spaces)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrEnumerateReferenceSpaces(	spaceCapacityInput,
													spaceCountOutput,
													spaces);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrCreateReferenceSpace(
		XrSession                                   session,
		const XrReferenceSpaceCreateInfo*			createInfo,
		XrSpace*									space)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrCreateReferenceSpace(	createInfo,
												space);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetReferenceSpaceBoundsRect(
		XrSession                                   session,
		XrReferenceSpaceType                        referenceSpaceType,
		XrExtent2Df*								bounds)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrGetReferenceSpaceBoundsRect(	referenceSpaceType,
														bounds);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrCreateActionSpace(
		XrSession                                   session,
		const XrActionSpaceCreateInfo*				createInfo,
		XrSpace*									space)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrCreateActionSpace(createInfo,
			space);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrLocateSpace(
		XrSpace                                     space,
		XrSpace                                     baseSpace,
		XrTime                                      time,
		XrSpaceLocation*							location)
	{
		FOXRVisionOSSpace* Space = (FOXRVisionOSSpace*)space;
		return Space->XrLocateSpace(baseSpace,
			time,
			location);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrDestroySpace(
		XrSpace                                     space)
	{
		FOXRVisionOSSpace* Space = (FOXRVisionOSSpace*)space;
		return Space->XrDestroySpace();
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateViewConfigurations(
		XrInstance                                  instance,
		XrSystemId                                  systemId,
		uint32_t                                    viewConfigurationTypeCapacityInput,
		uint32_t*									viewConfigurationTypeCountOutput,
		XrViewConfigurationType*					viewConfigurationTypes)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrEnumerateViewConfigurations(	systemId,
														viewConfigurationTypeCapacityInput,
														viewConfigurationTypeCountOutput,
														viewConfigurationTypes);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetViewConfigurationProperties(
		XrInstance                                  instance,
		XrSystemId                                  systemId,
		XrViewConfigurationType                     viewConfigurationType,
		XrViewConfigurationProperties*				configurationProperties)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrGetViewConfigurationProperties(	systemId,
															viewConfigurationType,
															configurationProperties);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateViewConfigurationViews(
		XrInstance                                  instance,
		XrSystemId                                  systemId,
		XrViewConfigurationType                     viewConfigurationType,
		uint32_t                                    viewCapacityInput,
		uint32_t*									viewCountOutput,
		XrViewConfigurationView*					views)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrEnumerateViewConfigurationViews(	systemId,
															viewConfigurationType,
															viewCapacityInput,
															viewCountOutput,
															views);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateSwapchainFormats(
		XrSession										session,
		uint32_t										formatCapacityInput,
		uint32_t*										formatCountOutput,
		int64_t*										formats)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrEnumerateSwapchainFormats(formatCapacityInput,
													formatCountOutput,
													formats);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrCreateSwapchain(
		XrSession										session,
		const XrSwapchainCreateInfo*					createInfo,
		XrSwapchain*									swapchain)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrCreateSwapchain(createInfo,
													swapchain);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrDestroySwapchain(
		XrSwapchain										swapchain)
	{
		OXRVISIONOS_CHECK_SWAPCHAIN_EARLY_RETURN(swapchain);
		FOXRVisionOSSwapchain* Swapchain = (FOXRVisionOSSwapchain*)swapchain;
		return Swapchain->XrDestroySwapchain();
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateSwapchainImages(
		XrSwapchain                                 swapchain,
		uint32_t                                    imageCapacityInput,
		uint32_t*									imageCountOutput,
		XrSwapchainImageBaseHeader*					images)
	{
		OXRVISIONOS_CHECK_SWAPCHAIN_EARLY_RETURN(swapchain);
		FOXRVisionOSSwapchain* Swapchain = (FOXRVisionOSSwapchain*)swapchain;
		return Swapchain->XrEnumerateSwapchainImages(
			imageCapacityInput,
			imageCountOutput,
			images);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrAcquireSwapchainImage(
		XrSwapchain                                 swapchain,
		const XrSwapchainImageAcquireInfo*			acquireInfo,
		uint32_t*									index)
	{
		OXRVISIONOS_CHECK_SWAPCHAIN_EARLY_RETURN(swapchain);
		FOXRVisionOSSwapchain* Swapchain = (FOXRVisionOSSwapchain*)swapchain;
		return Swapchain->XrAcquireSwapchainImage(
			acquireInfo,
			index);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrWaitSwapchainImage(
		XrSwapchain                                 swapchain,
		const XrSwapchainImageWaitInfo*				waitInfo)
	{
		OXRVISIONOS_CHECK_SWAPCHAIN_EARLY_RETURN(swapchain);
		FOXRVisionOSSwapchain* Swapchain = (FOXRVisionOSSwapchain*)swapchain;
		return Swapchain->XrWaitSwapchainImage(waitInfo);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrReleaseSwapchainImage(
		XrSwapchain                                 swapchain,
		const XrSwapchainImageReleaseInfo*			releaseInfo)
	{
		OXRVISIONOS_CHECK_SWAPCHAIN_EARLY_RETURN(swapchain);
		FOXRVisionOSSwapchain* Swapchain = (FOXRVisionOSSwapchain*)swapchain;
		return Swapchain->XrReleaseSwapchainImage(releaseInfo);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrBeginSession(
		XrSession                                   session,
		const XrSessionBeginInfo*					beginInfo)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		XrResult Result = Session->XrBeginSession(beginInfo);
		OXRVISIONOS_LOG_RESULT(Result);
		return Result;
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEndSession(
		XrSession                                   session)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		XrResult Result = Session->XrEndSession();
		OXRVISIONOS_LOG_RESULT(Result);
		return Result;
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrRequestExitSession(
		XrSession                                   session)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		XrResult Result = Session->XrRequestExitSession();
		OXRVISIONOS_LOG_RESULT(Result);
		return Result;
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrWaitFrame(
		XrSession                                   session,
		const XrFrameWaitInfo*						frameWaitInfo,
		XrFrameState*								frameState)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrWaitFrame(frameWaitInfo, frameState);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrBeginFrame(
		XrSession                                   session,
		const XrFrameBeginInfo*						frameBeginInfo)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrBeginFrame(frameBeginInfo);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEndFrame(
		XrSession                                   session,
		const XrFrameEndInfo*						frameEndInfo)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrEndFrame(frameEndInfo);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrLocateViews(
		XrSession                                   session,
		const XrViewLocateInfo*						viewLocateInfo,
		XrViewState*								viewState,
		uint32_t                                    viewCapacityInput,
		uint32_t*									viewCountOutput,
		XrView*										views)
	{
		//OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrLocateViews(viewLocateInfo,
			viewState,
			viewCapacityInput,
			viewCountOutput,
			views);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrStringToPath(
		XrInstance                                  instance,
		const char*									pathString,
		XrPath*										path)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrStringToPath(
			pathString,
			path);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrPathToString(
		XrInstance                                  instance,
		XrPath                                      path,
		uint32_t                                    bufferCapacityInput,
		uint32_t*									bufferCountOutput,
		char*										buffer)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrPathToString(
			path,
			bufferCapacityInput,
			bufferCountOutput,
			buffer);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrCreateActionSet(
		XrInstance                                  instance,
		const XrActionSetCreateInfo*				createInfo,
		XrActionSet*								actionSet)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrCreateActionSet(createInfo,
											actionSet);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrDestroyActionSet(
		XrActionSet                                 actionSet)
	{
		FOXRVisionOSActionSet* ActionSet = (FOXRVisionOSActionSet*)actionSet;
		return ActionSet->XrDestroyActionSet();
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrCreateAction(
		XrActionSet                                 actionSet,
		const XrActionCreateInfo*					createInfo,
		XrAction*									action)
	{
		FOXRVisionOSActionSet* ActionSet = (FOXRVisionOSActionSet*)actionSet;
		return ActionSet->XrCreateAction(createInfo,
										action);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrDestroyAction(
		XrAction                                    action)
	{
		FOXRVisionOSAction* Action = (FOXRVisionOSAction*)action;
		return Action->XrDestroyAction();
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrSuggestInteractionProfileBindings(
		XrInstance                                  instance,
		const XrInteractionProfileSuggestedBinding* suggestedBindings)
	{
		OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);
		FOXRVisionOSInstance* Instance = (FOXRVisionOSInstance*)instance;
		return Instance->XrSuggestInteractionProfileBindings(suggestedBindings);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrAttachSessionActionSets(
		XrSession                                   session,
		const XrSessionActionSetsAttachInfo*		attachInfo)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrAttachSessionActionSets(attachInfo);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetCurrentInteractionProfile(
		XrSession                                   session,
		XrPath                                      topLevelUserPath,
		XrInteractionProfileState*					interactionProfile)
	{
		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrGetCurrentInteractionProfile(topLevelUserPath, interactionProfile);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetActionStateBoolean(
		XrSession                                   session,
		const XrActionStateGetInfo*					getInfo,
		XrActionStateBoolean*						state)
	{
		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrGetActionStateBoolean(getInfo, state);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetActionStateFloat(
		XrSession                                   session,
		const XrActionStateGetInfo*					getInfo,
		XrActionStateFloat*							state)
	{
		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrGetActionStateFloat(getInfo, state);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetActionStateVector2f(
		XrSession                                   session,
		const XrActionStateGetInfo*					getInfo,
		XrActionStateVector2f*						state)
	{
		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrGetActionStateVector2f(getInfo, state);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetActionStatePose(
		XrSession                                   session,
		const XrActionStateGetInfo*					getInfo,
		XrActionStatePose*							state)
	{
		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrGetActionStatePose(getInfo, state);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrSyncActions(
		XrSession                                   session,
		const										XrActionsSyncInfo* syncInfo)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrSyncActions(syncInfo);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateBoundSourcesForAction(
		XrSession                                   session,
		const XrBoundSourcesForActionEnumerateInfo* enumerateInfo,
		uint32_t                                    sourceCapacityInput,
		uint32_t*									sourceCountOutput,
		XrPath*										sources)
	{
		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrEnumerateBoundSourcesForAction(
			enumerateInfo,
			sourceCapacityInput,
			sourceCountOutput,
			sources);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrGetInputSourceLocalizedName(
		XrSession                                   session,
		const XrInputSourceLocalizedNameGetInfo*	getInfo,
		uint32_t                                    bufferCapacityInput,
		uint32_t*									bufferCountOutput,
		char*										buffer)
	{
		// Note: we don't use this so I'm not implementing it.
		return XrResult::XR_ERROR_FUNCTION_UNSUPPORTED;
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrApplyHapticFeedback(
		XrSession                                   session,
		const XrHapticActionInfo*					hapticActionInfo,
		const XrHapticBaseHeader*					hapticFeedback)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);

		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrApplyHapticFeedback(
			hapticActionInfo,
			hapticFeedback);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrStopHapticFeedback(
		XrSession                                   session,
		const XrHapticActionInfo*					hapticActionInfo)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);
		
		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrStopHapticFeedback(
			hapticActionInfo);
	}

// XR_EXT_hand_tracking begin
	XRAPI_ATTR XrResult XRAPI_CALL xrCreateHandTrackerEXT(
		XrSession                                   session,
		const XrHandTrackerCreateInfoEXT*           createInfo,
		XrHandTrackerEXT*                           handTracker)
	{
		OXRVISIONOS_CHECK_SESSION_EARLY_RETURN(session);
		
		FOXRVisionOSSession* Session = (FOXRVisionOSSession*)session;
		return Session->XrCreateHandTrackerEXT(
			createInfo,
			handTracker);
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrDestroyHandTrackerEXT(
		XrHandTrackerEXT                            handTracker)
	{
		FOXRVisionOSSession::FOXRVisionOSHandTracker* HandTracker = (FOXRVisionOSSession::FOXRVisionOSHandTracker*)handTracker;
		return HandTracker->XrDestroyHandTrackerEXT();
	}

	XRAPI_ATTR XrResult XRAPI_CALL xrLocateHandJointsEXT(
		XrHandTrackerEXT                            handTracker,
		const XrHandJointsLocateInfoEXT*            locateInfo,
		XrHandJointLocationsEXT*                    locations)
	{
		FOXRVisionOSSession::FOXRVisionOSHandTracker* HandTracker = (FOXRVisionOSSession::FOXRVisionOSHandTracker*)handTracker;
		return HandTracker->XrLocateHandJointsEXT(
			locateInfo,
			locations);
	}
// XR_EXT_hand_tracking end

// Any supported extension functions would need to be mapped as well.  That could go here.

#define OXRVISIONOS_MAP_GLOBAL_FUNCTION(Type,Func) OXRVisionOS::GlobalFunctionMap.Add(FString(TEXT(#Func)), (PFN_xrVoidFunction)(OXRVisionOS::Func));
#define OXRVISIONOS_MAP_INSTANCE_FUNCTION(Type,Func) OXRVisionOS::InstanceFunctionMap.Add(FString(TEXT(#Func)), (PFN_xrVoidFunction)(OXRVisionOS::Func));
#define OXRVISIONOS_MAP_INSTANCE_FUNCTION_EXT(Func) OXRVisionOS::InstanceFunctionMap.Add(FString(TEXT(#Func)), (PFN_xrVoidFunction)(OXRVisionOS::Func));

	void BuildFunctionMaps()
	{
		ENUM_XR_ENTRYPOINTS_GLOBAL(OXRVISIONOS_MAP_GLOBAL_FUNCTION);
		ENUM_XR_ENTRYPOINTS(OXRVISIONOS_MAP_INSTANCE_FUNCTION);
		
		OXRVISIONOS_MAP_INSTANCE_FUNCTION_EXT(xrCreateHandTrackerEXT);
		OXRVISIONOS_MAP_INSTANCE_FUNCTION_EXT(xrDestroyHandTrackerEXT);
		OXRVISIONOS_MAP_INSTANCE_FUNCTION_EXT(xrLocateHandJointsEXT);
	}

#undef OXRVISIONOS_MAP_INSTANCE_FUNCTION
#undef OXRVISIONOS_MAP_GLOBAL_FUNCTION

	void ClearFunctionMaps()
	{
		GlobalFunctionMap.Reset();
		InstanceFunctionMap.Reset();
	}
}


IMPLEMENT_MODULE(FOXRVisionOS, OXRVisionOS)

void FOXRVisionOS::StartupModule()
{
	RegisterOpenXRExtensionModularFeature();
	IModularFeatures::Get().RegisterModularFeature(GetFeatureName(), this);

	OXRVisionOS::BuildFunctionMaps();

	SupportedExtensions.Add(XR_KHR_COMPOSITION_LAYER_DEPTH_EXTENSION_NAME, XR_KHR_composition_layer_depth_SPEC_VERSION);
	SupportedExtensions.Add(XR_EPIC_OXRVISIONOS_CONTROLLER_NAME, XR_EPIC_oxrvisionos_controller_SPEC_VERSION);
	SupportedExtensions.Add(XR_EXT_HAND_TRACKING_EXTENSION_NAME, XR_EXT_hand_tracking);
	//SupportedExtensions.Add(XR_EXT__EXTENSION_NAME, XR_MSFT_hand_interaction);
	//SupportedExtensions.Add(XR_EXT__EXTENSION_NAME, XR_EXT_eye_gaze_interaction_SPEC_VERSION);
}

void FOXRVisionOS::ShutdownModule()
{
	OXRVisionOS::ClearFunctionMaps();
	Instance = nullptr;
}

bool FOXRVisionOS::GetCustomLoader(PFN_xrGetInstanceProcAddr* OutGetProcAddr)
{
	*OutGetProcAddr = &OXRVisionOS::xrGetInstanceProcAddr;
	return true;
}

FOpenXRRenderBridge* FOXRVisionOS::GetCustomRenderBridge(XrInstance InInstance)
{
	RenderBridge = CreateRenderBridge_OXRVisionOS(InInstance);
	return RenderBridge;
}

void FOXRVisionOS::OnBeginRendering_GameThread(XrSession InSession, FSceneViewFamily& InViewFamily, TArrayView<const uint32> VisibleLayers)
{
    if (!FOXRVisionOSInstance::CheckSession(InSession))
    {
        return;
    }
    FOXRVisionOSSession* Session = (FOXRVisionOSSession*)InSession;
    Session->OnBeginRendering_GameThread();
}

void FOXRVisionOS::OnBeginRendering_RenderThread_PreDeviceLocationUpdate(XrSession InSession, FRDGBuilder& GraphBuilder)
{
	if (!FOXRVisionOSInstance::CheckSession(InSession))
	{
		return;
	}
	FOXRVisionOSSession* Session = (FOXRVisionOSSession*)InSession;
	Session->OnBeginRendering_RenderThread_PreDeviceLocationUpdate();
}

FOXRVisionOS* FOXRVisionOS::Get()
{
	TArray<FOXRVisionOS*> Impls = IModularFeatures::Get().GetModularFeatureImplementations<FOXRVisionOS>(GetFeatureName());

	// There can be only one!  Or zero.  The implementations are platform specific and we are not currently supporting 'overlapping' platforms.
	check(Impls.Num() <= 1);

	if (Impls.Num() > 0)
	{
		check(Impls[0]);
		return Impls[0];
	}
	return nullptr;
}

XrResult FOXRVisionOS::XrEnumerateApiLayerProperties(
	uint32_t  propertyCapacityInput,
	uint32_t* propertyCountOutput,
	XrApiLayerProperties* properties)
{
	if (properties == nullptr)
	{
		*propertyCountOutput = 0; //TODO make real if we need to support any layers
		return XrResult::XR_SUCCESS;
	}
	else
	{
		return XrResult::XR_ERROR_RUNTIME_FAILURE;
	}
}

bool FOXRVisionOS::IsExtensionSupported(const ANSICHAR* InName) const
{
	return SupportedExtensions.Contains(InName);
}
void FOXRVisionOS::PostSyncActions(XrSession InSession)
{
	if (!FOXRVisionOSInstance::CheckSession(InSession))
	{
		return;
	}
	FOXRVisionOSSession* Session = (FOXRVisionOSSession*)InSession;
	Session->OnPostSyncActions();
}

XrResult FOXRVisionOS::XrEnumerateInstanceExtensionProperties(
	const char* layerName,
	uint32_t propertyCapacityInput,
	uint32_t* propertyCountOutput,
	XrExtensionProperties* properties)
{
	if (propertyCountOutput == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	if (propertyCapacityInput != 0 && properties == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	*propertyCountOutput = SupportedExtensions.Num();
	if (propertyCapacityInput != 0)
	{
		if (propertyCapacityInput < *propertyCountOutput)
		{
			return XR_ERROR_SIZE_INSUFFICIENT;
		}

		int Index = 0;
		for (const auto& Pair : SupportedExtensions)
		{
			FCStringAnsi::Strncpy(properties[Index].extensionName, Pair.Key, XR_MAX_EXTENSION_NAME_SIZE);
			properties[Index].extensionVersion = Pair.Value;
			++Index;
		}
	}
	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOS::XrCreateInstance(
	const XrInstanceCreateInfo* createInfo,
	XrInstance* instance)
{
	// Only one instance supported
	if (Instance.IsValid())
	{
		return XrResult::XR_ERROR_LIMIT_REACHED;
	}

	XrResult Ret = FOXRVisionOSInstance::Create(Instance, createInfo, this);
	*instance = (XrInstance)Instance.Get();
	return Ret;

}

XrResult FOXRVisionOS::XrDestroyInstance(
	XrInstance instance)
{
	OXRVISIONOS_CHECK_INSTANCE_EARLY_RETURN(instance);

	Instance = nullptr;
	return XR_SUCCESS;
}
