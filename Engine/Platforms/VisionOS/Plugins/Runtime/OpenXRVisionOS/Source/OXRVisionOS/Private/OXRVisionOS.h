// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Modules/ModuleInterface.h"
#include "IOpenXRExtensionPlugin.h"

#include "Logging/LogMacros.h"

#include "OpenXRCore.h"
#include "Containers/Map.h"

// Uncomment to make intellisense work better.
//#include "../../../../../../../../Source/ThirdParty/OpenXR/include/openxr/openxr.h"

DECLARE_LOG_CATEGORY_EXTERN(LogOXRVisionOS, Log, All);
// Note: if you are turning this log up to verbose you may also want LogMetalVisionOS turned up to verbose.

static const TCHAR* VOSThreadString()
{
	if (IsInGameThread()) 
	{
		return TEXT("T~G");
	}
	else if (IsInRenderingThread())
	{
		return TEXT("T~R");
	}
	else if (IsInRHIThread())
	{
		return TEXT("T~I");
	}
	else
	{
		return TEXT("T~?");
	}
}

class FOXRVisionOSInstance;

class FOXRVisionOS : public IModuleInterface, public IOpenXRExtensionPlugin
{
public:
	/************************************************************************/
	/* IModuleInterface                                                     */
	/************************************************************************/
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/************************************************************************/
	/* IOpenXRExtensionPlugin                                               */
	/************************************************************************/
	virtual FString GetDisplayName() override
	{
		return FString(TEXT("OXRVisionOS"));
	}
	virtual bool GetCustomLoader(PFN_xrGetInstanceProcAddr* OutGetProcAddr) override;
	virtual bool IsStandaloneStereoOnlyDevice() override { return true; }
	virtual class FOpenXRRenderBridge* GetCustomRenderBridge(XrInstance InInstance) override;
	virtual bool RequiresRHIContext() const override { return true; }
    virtual void OnBeginRendering_GameThread(XrSession InSession, FSceneViewFamily& InViewFamily, TArrayView<const uint32> VisibleLayers) override;
	virtual void OnBeginRendering_RenderThread_PreDeviceLocationUpdate(XrSession InSession, FRDGBuilder& GraphBuilder) override;

	/************************************************************************/
	/* FOXRVisionOS                                                             */
	/************************************************************************/
	static FOXRVisionOS* Get();
	static const FOXRVisionOSInstance* GetInstance() { return Instance.Get(); }


	XrResult XrEnumerateApiLayerProperties(
		uint32_t  propertyCapacityInput,
		uint32_t* propertyCountOutput,
		XrApiLayerProperties* properties);

	XrResult XrEnumerateInstanceExtensionProperties(
		const char* layerName,
		uint32_t propertyCapacityInput,
		uint32_t* propertyCountOutput,
		XrExtensionProperties* properties);

	XrResult XrCreateInstance(
		const XrInstanceCreateInfo* createInfo,
		XrInstance* instance);

	XrResult XrDestroyInstance(
		XrInstance instance);

	static bool CheckInstance(XrInstance instance) { return (instance != XR_NULL_HANDLE) && (instance == (XrInstance)Instance.Get()); }

	FOpenXRRenderBridge* GetRenderBridge() const
	{
		return RenderBridge;
	}

	bool IsExtensionSupported(const ANSICHAR* InName) const;
	virtual void PostSyncActions(XrSession InSession) override;

private:
	static FName GetFeatureName()
	{
		static FName FeatureName = FName(TEXT("OXRVisionOS"));
		return FeatureName;
	}

private:
	struct FStringToVersionKeyFunc
	{
		typedef const ANSICHAR* KeyType;
		typedef const ANSICHAR* KeyInitType;
		typedef const TPairInitializer<const ANSICHAR*, int32_t>& ElementInitType;

		enum { bAllowDuplicateKeys = false };

		static FORCEINLINE bool Matches(const ANSICHAR* A, const ANSICHAR* B)
		{
			return TCString<ANSICHAR>::Stricmp(A, B) == 0;
		}

		static FORCEINLINE uint32 GetKeyHash(const ANSICHAR* Key)
		{
			return FCrc::Strihash_DEPRECATED(Key);
		}

		static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element.Key;
		}
	};	
	TMap<const ANSICHAR*, int32_t, FDefaultSetAllocator, FStringToVersionKeyFunc> SupportedExtensions;
	static TSharedPtr<FOXRVisionOSInstance, ESPMode::ThreadSafe> Instance;
	FOpenXRRenderBridge* RenderBridge = nullptr;
};
