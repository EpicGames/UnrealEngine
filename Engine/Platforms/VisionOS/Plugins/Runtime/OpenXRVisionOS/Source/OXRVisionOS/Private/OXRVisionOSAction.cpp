// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSAction.h"
#include "OXRVisionOSActionSet.h"
#include "OXRVisionOSInstance.h"
#include "OXRVisionOSController.h"
#include "OXRVisionOSTracker.h"
#include "HAL/PlatformMemory.h"

XrResult FOXRVisionOSAction::Create(TSharedPtr<FOXRVisionOSAction, ESPMode::ThreadSafe>& OutAction, const XrActionCreateInfo* createInfo, FOXRVisionOSActionSet* ActionSet)
{
	if (createInfo == nullptr || ActionSet == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (createInfo->type != XR_TYPE_ACTION_CREATE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	OutAction = MakeShared<FOXRVisionOSAction, ESPMode::ThreadSafe>(createInfo, ActionSet);
	if (OutAction->bCreateFailed)
	{
		OutAction = nullptr;
		return XrResult::XR_ERROR_RUNTIME_FAILURE;
	}

	return XrResult::XR_SUCCESS;
}

FOXRVisionOSAction::FOXRVisionOSAction(const XrActionCreateInfo* createInfo, FOXRVisionOSActionSet* InActionSet)
{
	ActionSet = InActionSet;

	check(createInfo->next == nullptr);
	ActionName = FName(createInfo->actionName);
	ActionType = createInfo->actionType;
	SubactionPathBoundSourcesAndStates.Add(XR_NULL_PATH);
	for (int i = 0; i < createInfo->countSubactionPaths; i++)
	{
		XrPath SubActionPath = createInfo->subactionPaths[i];
		SubActionPathBinding& NewBinding = SubactionPathBoundSourcesAndStates.Add(SubActionPath);
		FMemory::Memzero(&NewBinding.ActionState, sizeof(ActionStateUnion));
	}
	FCStringAnsi::Strncpy(LocalizedActionName, createInfo->localizedActionName, XR_MAX_LOCALIZED_ACTION_NAME_SIZE);
}

FOXRVisionOSAction::~FOXRVisionOSAction()
{
	if (bCreateFailed)
	{
		UE_LOGF(LogOXRVisionOS, Warning, "Destructing FOXRVisionOSAction because create failed.");
	}
}

XrResult FOXRVisionOSAction::XrDestroyAction()
{
	// ActionSet->DestroyAction() can delete this, so better just return after that.
	return ActionSet->DestroyAction(this);
}

FOXRVisionOSAction::EActionStateTranslation FOXRVisionOSAction::GetActionStateTranslation(EOXRVisionOSControllerButton Button) const
{
	if (ActionType == XR_ACTION_TYPE_BOOLEAN_INPUT && FOXRVisionOSController::IsFloatButton(Button))
	{
		return FOXRVisionOSAction::EActionStateTranslation::FloatAsBool;
	}

	return FOXRVisionOSAction::EActionStateTranslation::None;
}

void FOXRVisionOSAction::BindSource(XrPath AppPath, EOXRVisionOSControllerButton Button)
{
	BoundPaths.AddUnique(AppPath);

	char* PathString = ActionSet->GetInstance()->PathToString(AppPath);
	for (auto& Elem : SubactionPathBoundSourcesAndStates)
	{
		if (Elem.Key == XR_NULL_PATH)
		{
			// Null path queries all paths.
			Elem.Value.Buttons.Emplace(Button, GetActionStateTranslation(Button));
		}
		else
		{
			// If AppPath starts with any of the subaction paths for this action the binding also goes for that subaction path
			char* SubactionPathString = ActionSet->GetInstance()->PathToString(Elem.Key);
			int32 Len = FCStringAnsi::Strlen(SubactionPathString);
			if (0 == FCStringAnsi::Strncmp(SubactionPathString, PathString, Len))
			{
				Elem.Value.Buttons.Emplace(Button, GetActionStateTranslation(Button));
			}
		}
	}
}

const TArray<XrPath>& FOXRVisionOSAction::GetBoundSources() const
{
	return BoundPaths;
}

XrResult FOXRVisionOSAction::GetActionStateBoolean(XrPath SubActionPath, XrActionStateBoolean& OutState) const
{
	if (ActionType != XR_ACTION_TYPE_BOOLEAN_INPUT)
	{
		return XrResult::XR_ERROR_ACTION_TYPE_MISMATCH;
	}

	const SubActionPathBinding* Binding = SubactionPathBoundSourcesAndStates.Find(SubActionPath);
	if (Binding == nullptr)
	{
		return XrResult::XR_ERROR_PATH_UNSUPPORTED;
	}

	OutState = Binding->ActionState.Boolean;

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSAction::GetActionStateFloat(XrPath SubActionPath, XrActionStateFloat& OutState) const
{
	if (ActionType != XR_ACTION_TYPE_FLOAT_INPUT)
	{
		return XrResult::XR_ERROR_ACTION_TYPE_MISMATCH;
	}

	const SubActionPathBinding* Binding = SubactionPathBoundSourcesAndStates.Find(SubActionPath);
	if (Binding == nullptr)
	{
		return XrResult::XR_ERROR_PATH_UNSUPPORTED;
	}

	OutState = Binding->ActionState.Float;

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSAction::GetActionStateVector2f(XrPath SubActionPath, XrActionStateVector2f& OutState) const
{
	if (ActionType != XR_ACTION_TYPE_VECTOR2F_INPUT)
	{
		return XrResult::XR_ERROR_ACTION_TYPE_MISMATCH;
	}

	const SubActionPathBinding* Binding = SubactionPathBoundSourcesAndStates.Find(SubActionPath);
	if (Binding == nullptr)
	{
		return XrResult::XR_ERROR_PATH_UNSUPPORTED;
	}

	OutState = Binding->ActionState.Vector2f;

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSAction::GetActionStatePose(XrPath SubActionPath, XrActionStatePose& OutState) const
{
	if (ActionType != XR_ACTION_TYPE_POSE_INPUT)
	{
		return XrResult::XR_ERROR_ACTION_TYPE_MISMATCH;
	}

	const SubActionPathBinding* Binding = SubactionPathBoundSourcesAndStates.Find(SubActionPath);
	if (Binding == nullptr)
	{
		return XrResult::XR_ERROR_PATH_UNSUPPORTED;
	}

	OutState = Binding->ActionState.Pose;

	return XrResult::XR_SUCCESS;
}

void FOXRVisionOSAction::Sync(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, TSharedPtr <FOXRVisionOSTracker, ESPMode::ThreadSafe>& Tracker)
{
	for (auto& Elem : SubactionPathBoundSourcesAndStates)
	{
		ActionStateUnion & State = Elem.Value.ActionState;
		TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons = Elem.Value.Buttons;

		switch (ActionType)
		{
		case XR_ACTION_TYPE_BOOLEAN_INPUT:
			SyncPathsBoolean(Controllers, Buttons, State.Boolean);
			break;
		case XR_ACTION_TYPE_FLOAT_INPUT:
			SyncPathsFloat(Controllers, Buttons, State.Float);
			break;
		case XR_ACTION_TYPE_VECTOR2F_INPUT:
			SyncPathsVector2f(Controllers, Buttons, State.Vector2f);
			break;
		case XR_ACTION_TYPE_POSE_INPUT:
			SyncPathsPose(Tracker, Buttons, State.Pose);
			break;
		case XR_ACTION_TYPE_VIBRATION_OUTPUT:
			break;
		default:
			check(false);
			break;
		}
	}
}

void FOXRVisionOSAction::SyncPathsBoolean(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, const TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons, XrActionStateBoolean& OutActionState)
{
	// OpenXR spec says boolean input sources are combined as: OR

	bool bNewValue = false;
	bool bNewActive = false;
	XrTime NewTime = 0;
	for (const TPair<EOXRVisionOSControllerButton, EActionStateTranslation>& Button : Buttons)
	{
		bool bValue = false;
		bool bActive = false;
		XrTime Time = 0;
		switch (Button.Value)
		{
		case EActionStateTranslation::None:
			Controllers->GetActionStateBoolean(Button.Key, bValue, bActive, Time);
			break;
		case EActionStateTranslation::FloatAsBool:
		{
			float fValue;
			Controllers->GetActionStateFloat(Button.Key, fValue, bActive, Time);
			// Flip to false if true and value == 0, flip to true if false and value > 0.9.  We are detecting button release (which already has a deadzone) or nearly full press.
			bValue = OutActionState.currentState;
			if (bValue && fValue == 0.0f)
			{
				bValue = false;
			}
			else if(!bValue && fValue > 0.9f)
			{
				bValue = true;
			}
		}
			break;
		default:
			check(false);
			break;
		}
		if (bActive)
		{
			bNewActive = true;
			bNewValue |= bValue;
			NewTime = Time;
		}
	}
	OutActionState.isActive = bNewActive;
	if (OutActionState.isActive)
	{
		OutActionState.changedSinceLastSync = OutActionState.currentState != bNewValue;
		if (OutActionState.changedSinceLastSync)
		{
			OutActionState.lastChangeTime = NewTime;
		}
		OutActionState.currentState = bNewValue;
	}

	//if (OutActionState.changedSinceLastSync)
	//{
	//	UE_LOGF(LogOXRVisionOS, Log, "SyncPathsBool %ls Value: %i Timestamp: %lld", ANSI_TO_TCHAR(ActionName), OutActionState.currentState, OutActionState.lastChangeTime);
	//}
}

void FOXRVisionOSAction::SyncPathsFloat(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, const TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons, XrActionStateFloat& OutActionState)
{
	// OpenXR spec says float input sources are combined as: largest absolute value

	bool bNoValueYetRead = true;
	float fNewValue = 0.0f;
	bool bNewActive = false;
	XrTime NewTime = 0;
	for (const TPair<EOXRVisionOSControllerButton, EActionStateTranslation>& Button : Buttons)
	{
		float fValue = 0.0f;
		bool bActive = false;
		XrTime Time = 0;
		Controllers->GetActionStateFloat(Button.Key, fValue, bActive, Time);
		if (bActive)
		{
			bNewActive = true;
			if (bNoValueYetRead || FMath::Abs(fValue) > FMath::Abs(fNewValue))
			{
				bNoValueYetRead = false;
				fNewValue = fValue;
				NewTime = Time;
			}
		}
	}
	OutActionState.isActive = bNewActive;
	if (OutActionState.isActive)
	{
		OutActionState.changedSinceLastSync = OutActionState.currentState != fNewValue;
		if (OutActionState.changedSinceLastSync)
		{
			OutActionState.lastChangeTime = NewTime;
		}
		OutActionState.currentState = fNewValue;
	}

	//if (OutActionState.changedSinceLastSync)
	//{
	//	UE_LOGF(LogOXRVisionOS, Log, "SyncPathsFloat %ls Value: %0.2f Timestamp: %lld", ANSI_TO_TCHAR(ActionName), OutActionState.currentState, OutActionState.lastChangeTime);
	//}
}

void FOXRVisionOSAction::SyncPathsVector2f(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, const TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons, XrActionStateVector2f& OutActionState)
{
	// OpenXR spec says pose input sources are combined as: largest vector magnitude wins

	bool bNoValueYetRead = true;
	XrVector2f NewValue = { 0.0f, 0.0f };
	float NewMagnitudeSq = 0.0f;
	bool bNewActive = false;
	XrTime NewTime = 0;
	for (const TPair<EOXRVisionOSControllerButton, EActionStateTranslation>& Button : Buttons)
	{
		XrVector2f Value = {0.0f, 0.0f};
		bool bActive = false;
		XrTime Time = 0;
		Controllers->GetActionStateVector2f(Button.Key, Value, bActive, Time);
		if (bActive)
		{
			bNewActive = true;
			const float MagnitudeSq = (Value.x * Value.x) + (Value.y * Value.y);
			if (bNoValueYetRead || (MagnitudeSq > NewMagnitudeSq))
			{
				bNoValueYetRead = false;
				NewMagnitudeSq = MagnitudeSq;
				NewValue = Value;
				NewTime = Time;
			}
		}
	}
	OutActionState.isActive = bNewActive;
	if (OutActionState.isActive)
	{
		OutActionState.changedSinceLastSync = (OutActionState.currentState.x != NewValue.x) || (OutActionState.currentState.y != NewValue.y);
		if (OutActionState.changedSinceLastSync)
		{
			OutActionState.lastChangeTime = NewTime;
		}
		OutActionState.currentState = NewValue;
	}

	//if (OutActionState.changedSinceLastSync)
	//{
	//	UE_LOGF(LogOXRVisionOS, Log, "SyncPathsVector2f %ls Value: %0.2f,%0.2f Timestamp: %lld", ANSI_TO_TCHAR(ActionName), OutActionState.currentState.x, OutActionState.currentState.y, OutActionState.lastChangeTime);
	//}

}

void FOXRVisionOSAction::SyncPathsPose(TSharedPtr <FOXRVisionOSTracker, ESPMode::ThreadSafe>& Tracker, const TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons, XrActionStatePose& OutActionState)
{
	// OpenXR spec says pose input sources are combined as: always the same source

	bool bNewActive = false;
	for (const TPair<EOXRVisionOSControllerButton, EActionStateTranslation>& Button : Buttons)
	{
		bool bActive = false;
		Tracker->GetActionStatePose(Button.Key, bActive);
		if (bActive)
		{
			bNewActive = true;
		}
	}
	OutActionState.isActive = bNewActive;
}

EOXRVisionOSControllerButton FOXRVisionOSAction::GetPoseButton(XrPath SubactionPath) const
{
	const SubActionPathBinding* Binding = SubactionPathBoundSourcesAndStates.Find(SubactionPath);

	if (Binding)
	{
		if (Binding->Buttons.Num() > 0)
		{
			return Binding->Buttons[0].Key;  // just return the first button
		}
		else
		{
			return EOXRVisionOSControllerButton::NullInput;
		}
	}
	else
	{
		return EOXRVisionOSControllerButton::NullInput;
	}
}