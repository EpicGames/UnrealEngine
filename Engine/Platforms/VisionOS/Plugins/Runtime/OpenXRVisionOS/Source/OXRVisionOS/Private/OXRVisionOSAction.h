// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OXRVisionOS.h"

class FOXRVisionOSInstance;
class FOXRVisionOSActionSet;
class FOXRVisionOSController;
class FOXRVisionOSTracker;

enum class EOXRVisionOSControllerButton : int32;

class FOXRVisionOSAction
{
public:
	static XrResult Create(TSharedPtr<FOXRVisionOSAction, ESPMode::ThreadSafe>& OutAction, const XrActionCreateInfo* createInfo, FOXRVisionOSActionSet* ActionSet);
	FOXRVisionOSAction(const XrActionCreateInfo* createInfo, FOXRVisionOSActionSet* ActionSet);
	~FOXRVisionOSAction();
	XrResult XrDestroyAction();

	XrActionType GetActionType() const { return ActionType; }
	const FOXRVisionOSActionSet& GetActionSet() const { check(ActionSet); return *ActionSet; }

	void BindSource(XrPath AppPath, EOXRVisionOSControllerButton Button);
	const TArray<XrPath>& GetBoundSources() const;

	XrResult GetActionStateBoolean(XrPath SubActionPath, XrActionStateBoolean& OutState) const;
	XrResult GetActionStateFloat(XrPath SubActionPath, XrActionStateFloat& OutState) const;
	XrResult GetActionStateVector2f(XrPath SubActionPath, XrActionStateVector2f& OutState) const;
	XrResult GetActionStatePose(XrPath SubActionPath, XrActionStatePose& OutState) const;

	void Sync(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, TSharedPtr <FOXRVisionOSTracker, ESPMode::ThreadSafe>& Tracker);
	EOXRVisionOSControllerButton GetPoseButton(XrPath SubactionPath) const;

	FName GetActionName() const { return ActionName; }

private:
	bool bCreateFailed = false;
	FOXRVisionOSActionSet* ActionSet = nullptr;

	FName					ActionName;
	XrActionType			ActionType;
	char					LocalizedActionName[XR_MAX_LOCALIZED_ACTION_NAME_SIZE];

	TArray<XrPath> BoundPaths;
	// Note we craftily use XR_NULL_PATH as a subaction path to handle the case where none is specified on query.
	union ActionStateUnion
	{
		XrActionStateBoolean Boolean;
		XrActionStateFloat Float;
		XrActionStateVector2f Vector2f;
		XrActionStatePose Pose;
	};
	enum class EActionStateTranslation
	{
		None,
		FloatAsBool
	};
	struct SubActionPathBinding
	{
		TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>> Buttons;
		ActionStateUnion ActionState = {};
	};
	TMap<XrPath, SubActionPathBinding> SubactionPathBoundSourcesAndStates;
	EActionStateTranslation GetActionStateTranslation(EOXRVisionOSControllerButton Button) const;


	void SyncPathsBoolean(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, const TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons, XrActionStateBoolean& OutActionState);
	void SyncPathsFloat(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, const TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons, XrActionStateFloat& OutActionState);
	void SyncPathsVector2f(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, const TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons, XrActionStateVector2f& OutActionState);
	void SyncPathsPose(TSharedPtr <FOXRVisionOSTracker, ESPMode::ThreadSafe>& Tracker, const TArray<TPair<EOXRVisionOSControllerButton, EActionStateTranslation>>& Buttons, XrActionStatePose& OutActionState);
};
