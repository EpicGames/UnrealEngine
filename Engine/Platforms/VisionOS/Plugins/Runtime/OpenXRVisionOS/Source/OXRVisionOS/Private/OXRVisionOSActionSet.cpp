// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSActionSet.h"
#include "OXRVisionOSInstance.h"
#include "OXRVisionOSAction.h"
#include "OXRVisionOSController.h"

XrResult FOXRVisionOSActionSet::Create(TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>& OutActionSet, const XrActionSetCreateInfo* createInfo, FOXRVisionOSInstance* Instance)
{
	if (createInfo == nullptr || Instance == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (createInfo->type != XR_TYPE_ACTION_SET_CREATE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}


	OutActionSet = MakeShared<FOXRVisionOSActionSet, ESPMode::ThreadSafe>(createInfo, Instance);
	if (OutActionSet->bCreateFailed)
	{
		OutActionSet = nullptr;
		return XrResult::XR_ERROR_RUNTIME_FAILURE;
	}
	return XrResult::XR_SUCCESS;
}

FOXRVisionOSActionSet::FOXRVisionOSActionSet(const XrActionSetCreateInfo* createInfo, FOXRVisionOSInstance* InInstance)
{
	Instance = InInstance;

	CreateInfo = *createInfo;
}

FOXRVisionOSActionSet::~FOXRVisionOSActionSet()
{
	if (bCreateFailed)
	{
		UE_LOGF(LogOXRVisionOS, Warning, "Destructing FOXRVisionOSActionSet because session create failed.");
	}
}

XrResult FOXRVisionOSActionSet::XrDestroyActionSet()
{
	// Instance->DestroyActionSet() can delete this, so better just return after that.
	return Instance->DestroyActionSet(this);
}

XrResult FOXRVisionOSActionSet::XrCreateAction(
	const XrActionCreateInfo* createInfo,
	XrAction* action)
{
	if (IsAttached())
	{
		return XR_ERROR_ACTIONSETS_ALREADY_ATTACHED;
	}

	TSharedPtr<FOXRVisionOSAction, ESPMode::ThreadSafe> NewAction;
	XrResult Ret = FOXRVisionOSAction::Create(NewAction, createInfo, this);
	if (Ret == XrResult::XR_SUCCESS)
	{
		Actions.Add(NewAction);
		*action = (XrAction)NewAction.Get();
	}
	return Ret;
}

XrResult FOXRVisionOSActionSet::DestroyAction(FOXRVisionOSAction* Action)
{
	uint32 ArrayIndex = Actions.IndexOfByPredicate([Action](const TSharedPtr<FOXRVisionOSAction, ESPMode::ThreadSafe>& Data) { return (Data.Get() == Action); });

	if (ArrayIndex == INDEX_NONE)
	{
		return  XrResult::XR_ERROR_HANDLE_INVALID;
	}

	Actions.RemoveAtSwap(ArrayIndex);

	return XrResult::XR_SUCCESS;
}	

XrResult FOXRVisionOSActionSet::Attach(FOXRVisionOSSession* InSession)
{
	Session = InSession;

	return XrResult::XR_SUCCESS;
}

void FOXRVisionOSActionSet::ClearActive()
{
	bAllActive = false;
}

void FOXRVisionOSActionSet::SetActive(XrPath SubActionPath)
{
	bAllActive = true;

	if (SubActionPath != XR_NULL_PATH)
	{
		// OXRVisionOS does not currently correctly support action set per-subactionpath activation.
		check(false);
	}
}

void FOXRVisionOSActionSet::SyncActions(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, TSharedPtr <FOXRVisionOSTracker, ESPMode::ThreadSafe>& Tracker)
{
	if (bAllActive == false)
	{
		return;
	}

	for (TSharedPtr<FOXRVisionOSAction, ESPMode::ThreadSafe>& Action : Actions)
	{
		// Update the state
		Action->Sync(Controllers, Tracker);
	}
}
