// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OXRVisionOS.h"

class FOXRVisionOSInstance;
class FOXRVisionOSSession;
class FOXRVisionOSAction;
class FOXRVisionOSController;
class FOXRVisionOSTracker;

class FOXRVisionOSActionSet
{
public:
	static XrResult Create(TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>& OutActionSet, const XrActionSetCreateInfo* createInfo, FOXRVisionOSInstance* Instance);
	FOXRVisionOSActionSet(const XrActionSetCreateInfo* createInfo, FOXRVisionOSInstance* Instance);
	~FOXRVisionOSActionSet();
	XrResult XrDestroyActionSet();

	XrResult XrCreateAction(
		const XrActionCreateInfo* createInfo,
		XrAction* action);
	XrResult DestroyAction(FOXRVisionOSAction* Action);

	XrResult Attach(FOXRVisionOSSession* Session);
	bool IsAttached() const { return Session != nullptr; }

	const TArray<TSharedPtr<FOXRVisionOSAction, ESPMode::ThreadSafe>>& GetActions() const { return Actions;  }
	void ClearActive();
	void SetActive(XrPath SubActionPath);
	//uint32 GetPriority() const { return CreateInfo.priority; } //Note: action set priority currently unsupported
	void SyncActions(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& Controllers, TSharedPtr <FOXRVisionOSTracker, ESPMode::ThreadSafe>& Tracker);

	const FOXRVisionOSInstance* GetInstance() const { return Instance; }

private:
	bool bCreateFailed = false;
	FOXRVisionOSInstance* Instance = nullptr;
	FOXRVisionOSSession* Session = nullptr;
	XrActionSetCreateInfo CreateInfo;
	TArray<TSharedPtr<FOXRVisionOSAction, ESPMode::ThreadSafe>> Actions;
	bool bAllActive = false;
};
