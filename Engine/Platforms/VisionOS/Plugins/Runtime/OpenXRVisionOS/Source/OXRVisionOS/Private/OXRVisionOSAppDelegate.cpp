// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSAppDelegate.h"
#include "IOS/IOSAppDelegate.h"

CP_OBJECT_cp_layer_renderer* OXRVisionOS::GetSwiftLayerRenderer()
{
	return [IOSAppDelegate GetDelegate].SwiftLayer;
}

int OXRVisionOS::GetSwiftNumViewports()
{
	return  [IOSAppDelegate GetDelegate].SwiftLayerViewports.count;
}
CGRect OXRVisionOS::GetSwiftViewportRect(int Index)
{
	return  [[[IOSAppDelegate GetDelegate].SwiftLayerViewports objectAtIndex:Index] CGRectValue];
}

