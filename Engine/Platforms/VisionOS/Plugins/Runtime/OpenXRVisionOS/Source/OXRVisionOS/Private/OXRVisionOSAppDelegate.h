// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#import <CompositorServices/CompositorServices.h>

namespace OXRVisionOS
{
	CP_OBJECT_cp_layer_renderer* GetSwiftLayerRenderer();

	int GetSwiftNumViewports();

	CGRect GetSwiftViewportRect(int Index);
}
