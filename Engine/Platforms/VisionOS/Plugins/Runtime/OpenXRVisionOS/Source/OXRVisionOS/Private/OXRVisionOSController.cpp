// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSController.h"
#include "OXRVisionOSSession.h"
#include "OXRVisionOSInstance.h"
#include "OXRVisionOSTracker.h"
#include "OXRVisionOSPlatformUtils.h"
#include "Math/UnrealMathUtility.h"
//#include "HAL/PlatformMemory.h"
#include "Runtime/Launch/Resources/Version.h"

namespace OXRVisionOS
{
	const TCHAR* OXRVisionOSControllerButtonEnumNames[] =
	{
		TEXT("PushL"),

		TEXT("PushR"),

		TEXT("GripL"),
		TEXT("AimL"),
		TEXT("GripSurfaceL"),
		TEXT("GripR"),
		TEXT("AimR"),
		TEXT("GripSurfaceR"),
		TEXT("HMDPose"),

//		TEXT("GazePose"),

		TEXT("NullInput"),
	};
	static_assert((int32)(EOXRVisionOSControllerButton::NullInput) == UE_ARRAY_COUNT(OXRVisionOSControllerButtonEnumNames), "OXRVisionOSControllerButtonEnumNames does not match EOXRVisionOSControllerButton!");

	const TCHAR* OXRVisionOSControllerButtonToTCHAR(EOXRVisionOSControllerButton Button)
	{
		check((int32)Button >= 0);
		check((int32)Button <= (int32)EOXRVisionOSControllerButton::NullInput);
		return OXRVisionOSControllerButtonEnumNames[(int32)Button];
	}
}

//=============================================================================
bool VrControllerAPI::Init()
{
	return true;
}

//=============================================================================
void VrControllerAPI::Term()
{
}

//=============================================================================
int32 VrControllerAPI::ControllerOpen(EControllerHand ControllerHand)
{
	UE_LOGF(LogOXRVisionOS, Warning, "VrControllerAPI::ControllerOpen() : %d", static_cast<int32>(ControllerHand));

	return static_cast<int32>(ControllerHand);
}

//=============================================================================
void VrControllerAPI::ControllerClose(int32 Handle)
{
}

//=============================================================================
bool VrControllerAPI::ControllerIsReady(int32 Handle)
{
	return true;
}



/** FOXRVisionOSController implementation */
bool FOXRVisionOSController::Create(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& OutController, FOXRVisionOSSession* InSession)
{
	OutController = MakeShared<FOXRVisionOSController, ESPMode::ThreadSafe>(InSession);
	if (OutController->bCreateFailed)
	{
		OutController = nullptr;
		return false;
	}

	return true;
}

//=============================================================================
FOXRVisionOSController::FOXRVisionOSController(FOXRVisionOSSession* InSession)
{
	Session = InSession;

	FMemory::Memzero(&UserState, sizeof(UserState));

	VrControllerAPI::Init();
}

//=============================================================================
FOXRVisionOSController::~FOXRVisionOSController()
{
	VrControllerAPI::Term();
}

void FOXRVisionOSController::OnBeginSession(FOXRVisionOSTracker* Tracker)
{
	for (int i = 0; i < 2; ++i)
	{
		FControllerState& State = UserState.ControllerStates[i];

		const EControllerHand ControllerHand = ControllerIndexToHand(i);
		State.Handle = VrControllerAPI::ControllerOpen(ControllerHand);
		State.Hand = ControllerHand;

		//TODO register the device
		//Tracker->RegisterDevice(TrackerDeviceType, State.Handle);
	}
}

void FOXRVisionOSController::OnEndSession(FOXRVisionOSTracker* Tracker)
{
	for (int i = 0; i < 2; ++i)
	{
		const FControllerState& State = UserState.ControllerStates[i];
		Tracker->UnregisterDevice(State.Handle);
		VrControllerAPI::ControllerClose(State.Handle);
	}
}

int32 FOXRVisionOSController::GetLeftControllerHandle() const
{
	const FControllerState& State = UserState.ControllerStates[0];
	return State.Handle;
}

int32 FOXRVisionOSController::GetRightControllerHandle() const
{
	const FControllerState& State = UserState.ControllerStates[1];
	return State.Handle;
}

void FOXRVisionOSController::SyncActions()
{
	for (FControllerState& ControllerState : UserState.ControllerStates)
	{
		//int32_t Result;

		//Result = ReadControllerState(ControllerState.Handle, &ControllerState.);
		//checkf(Result == RESULT_OK, TEXT("ControllerReadState failed Result=%d"), Result);
	}
}

bool FOXRVisionOSController::GetActionStateBoolean(EOXRVisionOSControllerButton Button, bool& OutValue, bool &OutActive, XrTime& OutTimeStamp)
{
	FControllerState* ControllerState = nullptr;
	bool bIsTouch = false;
	switch (Button)
	{
	case EOXRVisionOSControllerButton::PushL:
		ControllerState = &UserState.ControllerStates[0];
		break;
	case EOXRVisionOSControllerButton::PushR:
		ControllerState = &UserState.ControllerStates[1];
		break;
	default:
		UE_LOGF(LogOXRVisionOS, Error, "GetActionStateBoolean unknown button %ls", OXRVisionOS::OXRVisionOSControllerButtonToTCHAR(Button));
		OutActive = false;
		check(false); // We should not fail to find a button here.
		return false;
	}

	OutActive = false; //TODO read the controller state ControllerState->connected;
	if (!OutActive)
	{
		return true;
	}

	//TODO: read the input state
	OutValue = false;
	return true;
}

// We are supporting treating boolean inputs as floats.
bool FOXRVisionOSController::GetActionStateBooleanAsFloat(EOXRVisionOSControllerButton Button, float& OutValue, bool& OutActive, XrTime& OutTimeStamp)
{
	bool bOutValue = false;
	if (GetActionStateBoolean(Button, bOutValue, OutActive, OutTimeStamp))
	{
		OutValue = bOutValue ? 1.0f : 0.0f;
		return true;
	}
	OutActive = false;
	return false;
}

#define TRIGGER_TO_FLOAT(x) ((float)(x) / 255.f)
#define ANALOG_TO_FLOAT(x) (((float)(x) - 128.0f) / 128.f)

bool FOXRVisionOSController::GetActionStateFloat(EOXRVisionOSControllerButton Button, float& OutValue, bool& OutActive, XrTime& OutTimeStamp)
{
	return GetActionStateBooleanAsFloat(Button, OutValue, OutActive, OutTimeStamp);
}

bool FOXRVisionOSController::GetActionStateVector2f(EOXRVisionOSControllerButton Button, XrVector2f& OutValue, bool& OutActive, XrTime& OutTimeStamp)
{
	OutActive = false;
	check(false);
	return false;
}

int32 FOXRVisionOSController::ControllerHandToIndex(EControllerHand ControllerHand)
{
	switch (ControllerHand)
	{
	default:
		checkNoEntry();
	case EControllerHand::Left:
		return LeftHandIndex;
	case EControllerHand::Right:
		return RightHandIndex;
	}
}

EControllerHand FOXRVisionOSController::ControllerIndexToHand(int32 ControllerIndex)
{
	switch (ControllerIndex)
	{
	default:
		checkNoEntry();
	case LeftHandIndex:
		return EControllerHand::Left;
	case RightHandIndex:
		return EControllerHand::Right;
	}
}

bool FOXRVisionOSController::IsFloatButton(EOXRVisionOSControllerButton Button)
{
	return false;
}
