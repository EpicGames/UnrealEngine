// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OpenXRCore.h"

class FOXRVisionOSSession;
class FOXRVisionOSTracker;
struct FHapticFeedbackBuffer;
struct FInputDeviceProperty;
enum class EControllerHand : uint8;

enum class EOXRVisionOSControllerButton : int32
{
	LeftButtonStart = 0,
	PushL = LeftButtonStart,
	LeftButtonCount,

	RightButtonStart = LeftButtonCount,
	PushR = RightButtonStart,
	RightButtonCount,

	PhysicalButtonCount = RightButtonCount,

	/** Max number of controller buttons */
	TotalButtonCount,

	/** Pose inputs, handled by OXRVisionOSTracker **/
	GripL = TotalButtonCount,
	AimL,
	GripSurfaceL,
	GripR,
	AimR,
	GripSurfaceR,
	HMDPose,

	/** The eye tracking input **/
//	GazePose,

	NullInput, // This input is never mapped to hardware.
};

namespace OXRVisionOS
{
	const TCHAR* OXRVisionOSControllerButtonToTCHAR(EOXRVisionOSControllerButton Button);
}

class VrControllerAPI
{
public:

	struct EVrControllerButton
	{
		static const int32 Mask[(int32)EOXRVisionOSControllerButton::PhysicalButtonCount];
	};

	/** VrControllerAPI Initialization and termination */
	static bool Init();
	static void Term();

	/** Controller methods */
	static int32 ControllerOpen(EControllerHand ControllerHand);
	static void ControllerClose(int32 Handle);
	static bool ControllerIsReady(int32 Handle);
};

/**
 * Implementation of the OXRVisionOS motion controllers.
 * One instance of this handles both controllers.
 */
class FOXRVisionOSController
{
public:
	static bool Create(TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe>& OutController, FOXRVisionOSSession* InSession);
	FOXRVisionOSController(FOXRVisionOSSession* InSession);
	~FOXRVisionOSController();

	void OnBeginSession(FOXRVisionOSTracker* Tracker);
	void OnEndSession(FOXRVisionOSTracker* Tracker);

	void SyncActions();
	bool GetActionStateBoolean(EOXRVisionOSControllerButton Button, bool& OutValue, bool& OutActive, XrTime& OutTimeStamp);
	bool GetActionStateFloat(EOXRVisionOSControllerButton Button, float& OutValue, bool& OutActive, XrTime& OutTimeStamp);
	bool GetActionStateVector2f(EOXRVisionOSControllerButton Button, XrVector2f& OutValue, bool& OutActive, XrTime& OutTimeStamp);

	int32 GetLeftControllerHandle() const;
	int32 GetRightControllerHandle() const;
	static bool IsFloatButton(EOXRVisionOSControllerButton Button);
	
private:
	static int32 ControllerHandToIndex(EControllerHand ControllerHand);
	static EControllerHand ControllerIndexToHand(int32 ControllerIndex);

	bool bCreateFailed = false;
	FOXRVisionOSSession* Session = nullptr;

	/** State for a controller */
	struct FControllerState
	{
		//TODO
		int32 Handle;

		/** Left or right hand which the controller represents */
		EControllerHand Hand;
	};

	static constexpr int32 LeftHandIndex = 0;
	static constexpr int32 RightHandIndex = 1;

	/** State for user connections */
	struct FUserState
	{
		/** Per controller state, 2 per user for left & right hands */
		FControllerState	ControllerStates[2];
	} UserState;

	/** Helper for using boolean inputs as float inputs*/
	bool GetActionStateBooleanAsFloat(EOXRVisionOSControllerButton Button, float& OutValue, bool& OutActive, XrTime& OutTimeStamp);
};

