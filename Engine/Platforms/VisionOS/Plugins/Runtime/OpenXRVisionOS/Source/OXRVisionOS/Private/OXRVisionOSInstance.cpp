// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSInstance.h"
#include "OXRVisionOSSession.h"
#include "OXRVisionOSActionSet.h"
#include "OXRVisionOSAction.h"
#include "OXRVisionOSController.h"
#include "Engine/Engine.h"
#include "OXRVisionOS_openxr_platform.h"
#include "OXRVisionOSAppDelegate.h"
#include "Misc/ConfigCacheIni.h"
#include "MetalRHIVisionOSBridge.h"

#define OXRVISIONOS_INSTANCE_NAME "OXRVisionOS"

// OXRVisionOS only supports one system.
#define OXRVISIONOS_XR_SYSTEM_ID 1
#define OXRVISIONOS_XR_SYSTEM_NAME "OXRVisionOSSystem"

#define OXRVISIONOS_RUNTIME_VERSION XR_MAKE_VERSION(0, 1, 0)

// Note: This is not a real registered vendorid, it is the khronos id + 1.
#define OXRVISIONOS_VENDOR_ID 0x10001

TSharedPtr<FOXRVisionOSSession, ESPMode::ThreadSafe> FOXRVisionOSInstance::Session;

namespace OXRVisionOSInstanceHelpers
{
	constexpr uint32 RecommendedPerEyeSwapchainWidth = 512;
	constexpr uint32 RecommendedPerEyeSwapchainHeight = 512;

	constexpr uint32 DefaultMaxPerEyeSwapchainWidth = 512;
	constexpr uint32 DefaultMaxPerEyeSwapchainHeight = 512;
}

XrResult FOXRVisionOSInstance::Create(TSharedPtr<FOXRVisionOSInstance, ESPMode::ThreadSafe>& OutInstance, const XrInstanceCreateInfo* CreateInfo, FOXRVisionOS* InModule)
{
	if (CreateInfo == nullptr || CreateInfo->type != XR_TYPE_INSTANCE_CREATE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	const XrApplicationInfo& AppInfo = CreateInfo->applicationInfo;

	if (AppInfo.applicationName[0] == 0)
	{
		return XrResult::XR_ERROR_NAME_INVALID;
	}

	//Could check api version here
	//createInfo->
	//XR_ERROR_API_VERSION_UNSUPPORTED

	//Could check/handle API layers
	//Note API layers generally do not require special handling.
	//uint32_t					createInfo.enabledApiLayerCount;
	//const char* const*		createInfo.enabledApiLayerNames;
	//XR_ERROR_API_LAYER_NOT_PRESENT

	const FOXRVisionOS* OXRVisionOS = FOXRVisionOS::Get();
	check(OXRVisionOS);
	for (int i = 0; i < CreateInfo->enabledExtensionCount; i++)
	{
		const char* ExtensionName = CreateInfo->enabledExtensionNames[i];
		if (!OXRVisionOS->IsExtensionSupported(ExtensionName))
		{
			UE_LOGF(LogOXRVisionOS, Error, "FOXRVisionOSInstance::Create required extension: %s but it is not supported.  Instance creation failed.", ExtensionName);
			return XrResult::XR_ERROR_EXTENSION_NOT_PRESENT;
		}
	}

	OutInstance = MakeShared<FOXRVisionOSInstance, ESPMode::ThreadSafe>(CreateInfo, InModule);
	if (OutInstance->bCreateFailed)
	{
		OutInstance = nullptr;
		return XrResult::XR_ERROR_RUNTIME_FAILURE;
	}
	return XrResult::XR_SUCCESS;
}

FOXRVisionOSInstance::FOXRVisionOSInstance(const XrInstanceCreateInfo* CreateInfo, FOXRVisionOS* InModule)
{
	Module = InModule;

	// Enable all requested extensions.  We already failed if any were unsupported.
	for (int i = 0; i < CreateInfo->enabledExtensionCount; i++)
	{
		const char* ExtensionName = CreateInfo->enabledExtensionNames[i];
		EnabledExtensions.Add(ExtensionName);
	}

	// Cache instance properties.
	InstanceProperties.type = XR_TYPE_INSTANCE_PROPERTIES;
	InstanceProperties.next = nullptr;
	InstanceProperties.runtimeVersion = OXRVISIONOS_RUNTIME_VERSION;
	FCStringAnsi::Strncpy(InstanceProperties.runtimeName, OXRVISIONOS_INSTANCE_NAME, XR_MAX_RUNTIME_NAME_SIZE);

	// Cache system properties.
	SystemProperties.type = XR_TYPE_SYSTEM_PROPERTIES;
	SystemProperties.next = nullptr;
	SystemProperties.systemId = OXRVISIONOS_XR_SYSTEM_ID;
	SystemProperties.vendorId = OXRVISIONOS_VENDOR_ID;
	FCStringAnsi::Strncpy(SystemProperties.systemName, OXRVISIONOS_XR_SYSTEM_NAME, XR_MAX_SYSTEM_NAME_SIZE);

	//TODO get graphics properties here
    const int NumViewports = OXRVisionOS::GetSwiftNumViewports();
    check(NumViewports > 0);
    check(NumViewports < 3); // unexpected, and we aren't handling it.
    CGRect LeftRect = OXRVisionOS::GetSwiftViewportRect(0);
    CGRect RightRect = NumViewports > 1 ? OXRVisionOS::GetSwiftViewportRect(1) : CGRectMake(0,0,LeftRect.size.width/4, LeftRect.size.height/4); //TODO CVars around this perhaps???

	int32 MaxWidth = LeftRect.size.width;
	int32 MaxHeight = LeftRect.size.height;

	SystemProperties.graphicsProperties.maxSwapchainImageWidth = MaxWidth;
    SystemProperties.graphicsProperties.maxSwapchainImageHeight = MaxHeight;
	//// OXRVisionOS version of OpenXR is not-compliant because we support less than XR_MIN_COMPOSITION_LAYERS_SUPPORTED
	//TODO temporarily just made up this value, but it may be correct.
	SystemProperties.graphicsProperties.maxLayerCount = 1;

	SystemProperties.trackingProperties.orientationTracking = XR_TRUE;
	SystemProperties.trackingProperties.positionTracking = XR_TRUE;

	// Create the null path
	{
		const char PathString[] = "XR_NULL_PATH";
		const SIZE_T PathSize = static_cast<SIZE_T>(sizeof(PathString));
		TUniquePtr<ANSICHAR[]> PathArrayElementPtr = MakeUnique<ANSICHAR[]>(PathSize);
		ANSICHAR* const CharPtr = PathArrayElementPtr.Get();
		TCString<ANSICHAR>::Strncpy(CharPtr, PathString, PathSize);
		uint32 NewIndex = XrPaths.Add(MoveTemp(PathArrayElementPtr));
		const XrPath NewPath = NewIndex;
		StringToXrPathMap.Add(CharPtr, NewPath);
	}

	//// Add additional system properties for extensions insert each one after the head struct.
	//if (IsExtensionEnabled(XR_EXT_EYE_GAZE_INTERACTION_EXTENSION_NAME))
	//{
	//	// Eye Gaze interaction
	//	EyeGazeInteractionSystemProperties.type = XR_TYPE_SYSTEM_EYE_GAZE_INTERACTION_PROPERTIES_EXT;
	//	EyeGazeInteractionSystemProperties.supportsEyeGazeInteraction = true;
	//	EyeGazeInteractionSystemProperties.next = SystemProperties.next;
	//	SystemProperties.next = &EyeGazeInteractionSystemProperties;
	//}

	OnWorldTickStartDelegateHandle = FWorldDelegates::OnWorldTickStart.AddRaw(this, &FOXRVisionOSInstance::OnWorldTickStart);
}

FOXRVisionOSInstance::~FOXRVisionOSInstance()
{
	FWorldDelegates::OnWorldTickStart.Remove(OnWorldTickStartDelegateHandle);

	if (bCreateFailed)
	{
		UE_LOGF(LogOXRVisionOS, Warning, "Destructing FOXRVisionOSInstance because instance create failed.");
	}
	else
	{
		//TODO instance loss logic?
	}
}

void FOXRVisionOSInstance::OnWorldTickStart(UWorld* World, ELevelTick TickType, float DeltaTime)
{
	//TODO
	// Poll device state to find out if the HMD is on the user's head
	// and send delegates about hmd worn status.  OpenXR hides this specific info from applications, the runtime is supposed to handle it.
	// But here we are the runtime so we need to make it possible to deal with this.
	// We may need to drive the session to idle when the hmd is off, and return it to ready when the hmd is on.   See FOXRVisionOSSession:OnWorldTickStart.
	// 		const bool NewbHMDWorn = TODO_PLATFORM_API_HMD_WORN;
	// 		if (NewbHMDWorn != bHMDWorn)
	// 		{
	// 			bHMDWorn = NewbHMDWorn;
	// 			if (bHMDWorn)
	// 			{
	// 				FCoreDelegates::VRHeadsetPutOnHead.Broadcast();
	// 			}
	// 			else
	// 			{
	// 				FCoreDelegates::VRHeadsetRemovedFromHead.Broadcast();
	// 			}
	// 		}
}

XrResult FOXRVisionOSInstance::XrGetSystem(
	const XrSystemGetInfo* getInfo,
	XrSystemId* systemId)
{
	check(bCreateFailed == false);

	if (getInfo == nullptr || getInfo->type != XR_TYPE_SYSTEM_GET_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (getInfo->formFactor != XR_FORM_FACTOR_HEAD_MOUNTED_DISPLAY)
	{
		return XrResult::XR_ERROR_FORM_FACTOR_UNSUPPORTED;
	}

	if (!IsHMDAvailable())
	{
		return XR_ERROR_FORM_FACTOR_UNAVAILABLE;
	}

	*systemId = OXRVISIONOS_XR_SYSTEM_ID;
	return XrResult::XR_SUCCESS;
}

bool FOXRVisionOSInstance::IsHMDAvailable()
{
	// We are running on the device which is the hmd, so it can't be unavailable.
	return true;
}

XrResult FOXRVisionOSInstance::XrGetInstanceProperties(
	XrInstanceProperties* OutInstanceProperties)
{
	if (OutInstanceProperties == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	*OutInstanceProperties = InstanceProperties;
	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSInstance::XrGetSystemProperties(
	XrSystemId                                  SystemId,
	XrSystemProperties*							Properties)
{
	if (SystemId != OXRVISIONOS_XR_SYSTEM_ID)
	{
		return XrResult::XR_ERROR_SYSTEM_INVALID;
	}
	if (Properties == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	
	// Cache the next ptr, so we don't overwrite it with all the other system properties.
	void* NextTmp = Properties->next;
	*Properties = SystemProperties;
	Properties->next = NextTmp;
	
	XrSystemHandTrackingPropertiesEXT* HandTrackingProperties = OpenXR::FindChainedStructByType<XrSystemHandTrackingPropertiesEXT>(Properties, XR_TYPE_SYSTEM_HAND_TRACKING_PROPERTIES_EXT);
	if (HandTrackingProperties) {
		HandTrackingProperties->supportsHandTracking = XR_TRUE;
	}

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSInstance::XrEnumerateEnvironmentBlendModes(
	XrSystemId                                  SystemId,
	XrViewConfigurationType                     ViewConfigurationType,
	uint32_t                                    EnvironmentBlendModeCapacityInput,
	uint32_t*									EnvironmentBlendModeCountOutput,
	XrEnvironmentBlendMode*						EnvironmentBlendModes)
{
	if (SystemId != OXRVISIONOS_XR_SYSTEM_ID)
	{
		return XrResult::XR_ERROR_SYSTEM_INVALID;
	}
	if (EnvironmentBlendModeCountOutput == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	if (EnvironmentBlendModeCapacityInput != 0 && EnvironmentBlendModes == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	switch (ViewConfigurationType)
	{
	//case XR_VIEW_CONFIGURATION_TYPE_PRIMARY_MONO: // TODO add or delete this.
	case  XrViewConfigurationType::XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO:
	{
		*EnvironmentBlendModeCountOutput = 1;
		if (EnvironmentBlendModeCapacityInput != 0)
		{
			check(EnvironmentBlendModeCapacityInput >= 1); // Because we only need 1 we can't ever need to return XR_ERROR_SIZE_INSUFFICIENT

			EnvironmentBlendModes[0] = XR_ENVIRONMENT_BLEND_MODE_ALPHA_BLEND;
		}
		return XrResult::XR_SUCCESS;
	}
	default:
	{
		*EnvironmentBlendModeCountOutput = 0;
		return XrResult::XR_ERROR_VIEW_CONFIGURATION_TYPE_UNSUPPORTED;
	}
	}
}

XrResult FOXRVisionOSInstance::XrEnumerateViewConfigurations(
	XrSystemId                                  SystemId,
	uint32_t                                    ViewConfigurationTypeCapacityInput,
	uint32_t*									ViewConfigurationTypeCountOutput,
	XrViewConfigurationType*					ViewConfigurationTypes)
{
	if (SystemId != OXRVISIONOS_XR_SYSTEM_ID)
	{
		return XrResult::XR_ERROR_SYSTEM_INVALID;
	}
	if (ViewConfigurationTypeCountOutput == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	if (ViewConfigurationTypeCapacityInput != 0 && ViewConfigurationTypes == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	*ViewConfigurationTypeCountOutput = 1;
	if (ViewConfigurationTypeCapacityInput != 0)
	{
		check(ViewConfigurationTypeCapacityInput >= 1); // Because we only need 1 we can't ever need to return XR_ERROR_SIZE_INSUFFICIENT

		ViewConfigurationTypes[0] = XrViewConfigurationType::XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO;
	}
	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSInstance::XrGetViewConfigurationProperties(
	XrSystemId                                  SystemId,
	XrViewConfigurationType                     ViewConfigurationType,
	XrViewConfigurationProperties*				ConfigurationProperties)
{
	if (SystemId != OXRVISIONOS_XR_SYSTEM_ID)
	{
		return XrResult::XR_ERROR_SYSTEM_INVALID;
	}
	if (ConfigurationProperties == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (ViewConfigurationType == XrViewConfigurationType::XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO)
	{
		ConfigurationProperties->type = XR_TYPE_VIEW_CONFIGURATION_PROPERTIES;
		ConfigurationProperties->next = nullptr;
		ConfigurationProperties->viewConfigurationType = XrViewConfigurationType::XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO;
		ConfigurationProperties->fovMutable = XR_FALSE;
		return XrResult::XR_SUCCESS;
	}
	else
	{
		return XrResult::XR_ERROR_VIEW_CONFIGURATION_TYPE_UNSUPPORTED;
	}
}

XrResult FOXRVisionOSInstance::XrEnumerateViewConfigurationViews(
	XrSystemId                                  SystemId,
	XrViewConfigurationType                     ViewConfigurationType,
	uint32_t                                    ViewCapacityInput,
	uint32_t*									ViewCountOutput,
	XrViewConfigurationView*					Views)
{
	if (SystemId != OXRVISIONOS_XR_SYSTEM_ID)
	{
		return XrResult::XR_ERROR_SYSTEM_INVALID;
	}
	if (ViewCountOutput == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	if (ViewCapacityInput != 0 && Views == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (ViewConfigurationType == XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO)
	{
		*ViewCountOutput = 2;
		if (ViewCapacityInput != 0)
		{
			if (ViewCapacityInput < *ViewCountOutput)
			{
				return XR_ERROR_SIZE_INSUFFICIENT;
			}

			XrViewConfigurationView View;
			View.type = XR_TYPE_VIEW_CONFIGURATION_VIEW;
			View.next = nullptr;
			View.recommendedImageRectWidth			= SystemProperties.graphicsProperties.maxSwapchainImageWidth;
			View.recommendedImageRectHeight			= SystemProperties.graphicsProperties.maxSwapchainImageHeight;
			View.maxImageRectWidth					= SystemProperties.graphicsProperties.maxSwapchainImageWidth;
			View.maxImageRectHeight					= SystemProperties.graphicsProperties.maxSwapchainImageHeight;
			View.recommendedSwapchainSampleCount	= 1; //TODO: Is this right?
			View.maxSwapchainSampleCount			= 1; //TODO: Is this right?

			Views[0] = View;
			Views[1] = View;
		}
		return XrResult::XR_SUCCESS;
	}
	else
	{
		return XrResult::XR_ERROR_VIEW_CONFIGURATION_TYPE_UNSUPPORTED;
	}
}

XrResult FOXRVisionOSInstance::XrCreateSession(
	const XrSessionCreateInfo*		CreateInfo,
	XrSession*						OutSession)
{
	// Only one session supported.
	if (Session.IsValid())
	{
		return XrResult::XR_ERROR_LIMIT_REACHED;
	}

	XrResult Ret = FOXRVisionOSSession::Create(Session, CreateInfo, this);
	*OutSession = (XrSession)Session.Get();
	return Ret;
}

XrResult FOXRVisionOSInstance::XrStringToPath(
	const char*				PathString,
	XrPath*					Path)
{
	if (PathString == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	// If this path exists return it.
	XrPath* Found = StringToXrPathMap.Find(PathString);
	if (Found != nullptr)
	{
		*Path = *Found;
		return XrResult::XR_SUCCESS;
	}

	// Validate the pathString
	if (PathString[0] != '/')
	{
		return XR_ERROR_PATH_FORMAT_INVALID;
	}
	uint32 PathSize = FCStringAnsi::Strlen(PathString) + 1;
	if (PathSize > XR_MAX_PATH_LENGTH)
	{
		return XR_ERROR_PATH_FORMAT_INVALID;
	}


	// Create a new path and return it..
	*Path = StringToPath(PathString, PathSize);
	return XrResult::XR_SUCCESS;
}

XrPath FOXRVisionOSInstance::StringToPath(const char* PathString)
{
	uint32 PathSize = FCStringAnsi::Strlen(PathString) + 1;
	return StringToPath(PathString, PathSize);
}
XrPath FOXRVisionOSInstance::StringToPath(const char* PathString, uint32 PathSize)
{
	check(PathSize <= XR_MAX_PATH_LENGTH);

	XrPath* FoundPath = StringToXrPathMap.Find(PathString);
	if (FoundPath)
	{
		return *FoundPath;
	}

	// Create a new path and return it..
	TUniquePtr<ANSICHAR[]> PathArrayElementPtr = MakeUnique<ANSICHAR[]>(PathSize);
	ANSICHAR* const CharPtr = PathArrayElementPtr.Get();
	TCString<ANSICHAR>::Strncpy(CharPtr, PathString, PathSize);
	uint32 NewIndex = XrPaths.Add(MoveTemp(PathArrayElementPtr));
	const XrPath NewPath = NewIndex;
	StringToXrPathMap.Add(CharPtr, NewPath);
	return NewPath;
}

XrResult FOXRVisionOSInstance::XrPathToString(
	XrPath                                      Path,
	uint32_t									BufferCapacityInput,
	uint32_t*									BufferCountOutput,
	char*										Buffer)
{
	if (Path == XR_NULL_PATH)
	{
		return XrResult::XR_ERROR_HANDLE_INVALID;
	}

	const uint32_t Index = Path;

	if (Index > XrPaths.Num())
	{
		return XrResult::XR_ERROR_HANDLE_INVALID;
	}

	const uint32_t PathSize = FCStringAnsi::Strlen(XrPaths[Index].Get()) + 1;
	*BufferCountOutput = PathSize;
	if (BufferCapacityInput == 0)
	{
		return XrResult::XR_SUCCESS;
	}

	if (BufferCapacityInput < PathSize)
	{
		return XrResult::XR_ERROR_SIZE_INSUFFICIENT;
	}

	if (Buffer == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	ANSICHAR* PathChars = XrPaths[Index].Get();
	FCStringAnsi::Strncpy(Buffer, PathChars, BufferCapacityInput);

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSInstance::XrCreateActionSet(
	const XrActionSetCreateInfo* CreateInfo,
	XrActionSet* ActionSet)
{
	TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe> NewActionSet;
	XrResult Ret = FOXRVisionOSActionSet::Create(NewActionSet, CreateInfo, this);
	if (Ret == XrResult::XR_SUCCESS)
	{
		ActionSets.Add(NewActionSet);
		*ActionSet = (XrActionSet)NewActionSet.Get();
	}
	return Ret;
}

XrResult FOXRVisionOSInstance::DestroyActionSet(class FOXRVisionOSActionSet* ActionSet)
{
	uint32 ArrayIndex = ActionSets.IndexOfByPredicate([ActionSet](const TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>& Data) { return (Data.Get() == ActionSet); });

	if (ArrayIndex == INDEX_NONE)
	{
		return  XrResult::XR_ERROR_HANDLE_INVALID;
	}

	ActionSets.RemoveAtSwap(ArrayIndex);

	return XrResult::XR_SUCCESS;
}

TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe> FOXRVisionOSInstance::GetActionSetSharedPtr(XrActionSet ActionSet)
{
	for (TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>& ActionSetPtr : ActionSets)
	{
		if (reinterpret_cast<XrActionSet>(ActionSetPtr.Get()) == ActionSet)
		{
			return ActionSetPtr;
		}
	}
	return nullptr;
}

void FOXRVisionOSInstance::CreateInteractionProfiles()
{
	if (InteractionProfiles.Num() != 0)
	{
		return;
	}

	// Note: Input profiles are defined in order of decreasing preference.  when bindings for multiple profiles are provided each top level path will use the highest preference input profile.
	// Thus visionos_controller goes first, any supported emulations should be in the middle with better emulations earlier, simple_controller and extensions go last.

	if (IsExtensionEnabled(XR_EPIC_OXRVISIONOS_CONTROLLER_NAME))
	{
		const XrPath InteractionProfilePath = StringToPath("/interaction_profiles/epic/visionos_controller");
		PreferredInteractionProfileBindingList.Add(InteractionProfilePath);
		TMap<XrPath, EOXRVisionOSControllerButton>& Pairs = InteractionProfiles.Add(InteractionProfilePath);

		// The native profile just maps the paths to their native source button.
		Pairs.Add(StringToPath("/user/hand/left/input/select/click"), EOXRVisionOSControllerButton::PushL);  // stick click
		Pairs.Add(StringToPath("/user/hand/right/input/select/click"), EOXRVisionOSControllerButton::PushR);
		Pairs.Add(StringToPath("/user/hand/left/input/grip/pose"), EOXRVisionOSControllerButton::GripL);
		Pairs.Add(StringToPath("/user/hand/right/input/grip/pose"), EOXRVisionOSControllerButton::GripR);
		Pairs.Add(StringToPath("/user/hand/left/input/aim/pose"), EOXRVisionOSControllerButton::AimL);
		Pairs.Add(StringToPath("/user/hand/right/input/aim/pose"), EOXRVisionOSControllerButton::AimR);
		Pairs.Add(StringToPath("/user/hand/left/input/grip_surface/pose"), EOXRVisionOSControllerButton::GripSurfaceL);
		Pairs.Add(StringToPath("/user/hand/right/input/grip_surface/pose"), EOXRVisionOSControllerButton::GripSurfaceR);

		// Parent Paths  One can use the 'parent' path of a value or click and it should behave like click if there is one and value if there is not.  OpenXR spec 11.4.
		Pairs.Add(StringToPath("/user/hand/left/input/select"), EOXRVisionOSControllerButton::PushL);			// stick click
		Pairs.Add(StringToPath("/user/hand/right/input/select"), EOXRVisionOSControllerButton::PushR);
	}

	// Right now I'll assume we are too simple for simple controller.
	//{
	//	const XrPath InteractionProfilePath = StringToPath("/interaction_profiles/khr/simple_controller");
	//	PreferredInteractionProfileBindingList.Add(InteractionProfilePath);
	//	TMap<XrPath, EOXRVisionOSControllerButton>& Pairs = InteractionProfiles.Add(InteractionProfilePath);

	//	Pairs.Add(StringToPath("/user/hand/left/input/select/click"), EOXRVisionOSControllerButton::PushL);
	//	Pairs.Add(StringToPath("/user/hand/left/input/menu/click"), EOXRVisionOSControllerButton::NullInput);			// This breaks the simplecontroller input pretty badly have to figure something out for this to use this config
	//	Pairs.Add(StringToPath("/user/hand/left/input/grip/pose"), EOXRVisionOSControllerButton::GripL);
	//	Pairs.Add(StringToPath("/user/hand/left/input/aim/pose"), EOXRVisionOSControllerButton::AimL);
	//	Pairs.Add(StringToPath("/user/hand/left/output/haptic"), EOXRVisionOSControllerButton::NullInput);				// haptics - we do not access haptics through these, so NullInput

	//	Pairs.Add(StringToPath("/user/hand/right/input/select/click"), EOXRVisionOSControllerButton::PushR);
	//	Pairs.Add(StringToPath("/user/hand/right/input/menu/click"), EOXRVisionOSControllerButton::NullInput);			// Intentionally not mapping a right button for this.
	//	Pairs.Add(StringToPath("/user/hand/right/input/grip/pose"), EOXRVisionOSControllerButton::GripR);
	//	Pairs.Add(StringToPath("/user/hand/right/input/aim/pose"), EOXRVisionOSControllerButton::AimR);
	//	Pairs.Add(StringToPath("/user/hand/right/output/haptic"), EOXRVisionOSControllerButton::NullInput);				// haptics - we do not access haptics through these, so NullInput


	//	// Parent Paths  One can use the 'parent' path of a value or click and it should behave like click if there is one and value if there is not.  OpenXR spec 11.4.
	//	Pairs.Add(StringToPath("/user/hand/left/input/select"), EOXRVisionOSControllerButton::Square);
	//	Pairs.Add(StringToPath("/user/hand/left/input/menu"), EOXRVisionOSControllerButton::NullInput);

	//	Pairs.Add(StringToPath("/user/hand/right/input/select"), EOXRVisionOSControllerButton::Cross);
	//	Pairs.Add(StringToPath("/user/hand/right/input/menu"), EOXRVisionOSControllerButton::NullInput);				// Intentionally not mapping a right button for this.
	//}

	// if (IsExtensionEnabled(XR_EXT_EYE_GAZE_INTERACTION_EXTENSION_NAME))
	// {
	// 	const XrPath InteractionProfilePath = StringToPath("/interaction_profiles/ext/eye_gaze_interaction");
	// 	PreferredInteractionProfileBindingList.Add(InteractionProfilePath);
	// 	TMap<XrPath, EOXRVisionOSControllerButton>& Pairs = InteractionProfiles.Add(InteractionProfilePath);

	// 	Pairs.Add(StringToPath("/user/eyes_ext/input/gaze_ext/pose"), EOXRVisionOSControllerButton::GazePose);
	// }

	// Define all supported top level paths
	TopLevelPathInteractionProfileMap.Add(StringToPath("/user/hand/left"), XR_NULL_PATH);
	TopLevelPathInteractionProfileMap.Add(StringToPath("/user/hand/right"), XR_NULL_PATH);
	TopLevelPathInteractionProfileMap.Add(StringToPath("/user/head"), XR_NULL_PATH);
	TopLevelPathInteractionProfileMap.Add(StringToPath("/user/eyes_ext"), XR_NULL_PATH);

	// Build map of all input paths to top level paths
	for (const auto& Pair : TopLevelPathInteractionProfileMap)
	{
		const XrPath TopLevelPath = Pair.Key;
		const ANSICHAR* TopLevelPathChars = XrPaths[TopLevelPath].Get();
		const int32 TopLevelPathCharsLen = FCStringAnsi::Strlen(TopLevelPathChars);

		for (const auto& Profile : InteractionProfiles)
		{
			for (const auto& Pair2 : Profile.Value)
			{
				XrPath InputPath = Pair2.Key;
				if (!InputPathToTopLevelPathMap.Contains(InputPath))
				{
					ANSICHAR* InputPathChars = XrPaths[InputPath].Get();
					if (FCStringAnsi::Strncmp(TopLevelPathChars, InputPathChars, TopLevelPathCharsLen) == 0)
					{
						InputPathToTopLevelPathMap.Add(InputPath, TopLevelPath);
					}
				}
			}
		}
	}
}

bool FOXRVisionOSInstance::IsFirstInputProfilePreferred(XrPath First, XrPath Second) const
{
	if (Second == XR_NULL_PATH)
	{
		return true;
	}
	int32 FirstIndex = PreferredInteractionProfileBindingList.Find(First);
	int32 SecondIndex = PreferredInteractionProfileBindingList.Find(Second);
	check(FirstIndex != INDEX_NONE);
	check(SecondIndex != INDEX_NONE);
	return FirstIndex <= SecondIndex;
}

XrResult FOXRVisionOSInstance::XrSuggestInteractionProfileBindings(
	const XrInteractionProfileSuggestedBinding* SuggestedBindings)
{
	if (SuggestedBindings == nullptr 
		|| SuggestedBindings->type != XR_TYPE_INTERACTION_PROFILE_SUGGESTED_BINDING
		|| SuggestedBindings->countSuggestedBindings <= 0
		|| SuggestedBindings->suggestedBindings == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}	

	UE_LOGF(LogHMD, Verbose, "XrSuggestInteractionProfileBindings profile: %ls", ANSI_TO_TCHAR(PathToString(SuggestedBindings->interactionProfile)));

	CreateInteractionProfiles();


	const TMap<XrPath, EOXRVisionOSControllerButton>* InteractionProfilePairs = InteractionProfiles.Find(SuggestedBindings->interactionProfile);
	if (InteractionProfilePairs == nullptr)
	{
		// We are successfully ignoring this interaction profile.
		UE_LOGF(LogOXRVisionOS, Verbose, "  Ignoring %ls because OXRVisionOS does not support it", ANSI_TO_TCHAR(PathToString(SuggestedBindings->interactionProfile)));
		return XrResult::XR_SUCCESS;
	}

	FInteractionProfileBinding* InteractionProfileBinding = InteractionProfileBindings.Find(SuggestedBindings->interactionProfile);
	if (InteractionProfileBinding)
	{
		// Previous bindings are overwritten.
		InteractionProfileBinding->Bindings.Empty(SuggestedBindings->countSuggestedBindings);
	}
	else
	{
		InteractionProfileBinding = &InteractionProfileBindings.Add(SuggestedBindings->interactionProfile);
		InteractionProfileBinding->InteractionProfilePath = SuggestedBindings->interactionProfile;
		InteractionProfileBinding->Bindings.Reserve(SuggestedBindings->countSuggestedBindings);
	}

	const uint32_t SuggestedBindingCount = SuggestedBindings->countSuggestedBindings;
	for (uint32_t i = 0; i < SuggestedBindingCount; i++)
	{
		const XrActionSuggestedBinding& Binding = SuggestedBindings->suggestedBindings[i];
		// Map to the native button
		const EOXRVisionOSControllerButton* Source = InteractionProfilePairs->Find(Binding.binding);
		if (Source == nullptr)
		{
			UE_LOGF(LogOXRVisionOS, Warning, "Unsupported path %ls in profile %ls", ANSI_TO_TCHAR(PathToString(Binding.binding)), ANSI_TO_TCHAR(PathToString(SuggestedBindings->interactionProfile)));
			return XrResult::XR_ERROR_PATH_UNSUPPORTED;
		}
		else
		{
			EOXRVisionOSControllerButton ButtonSource = *Source;
			FOXRVisionOSAction* Action = (FOXRVisionOSAction*)Binding.action;
			if (Action->GetActionSet().IsAttached())
			{
				return XrResult::XR_ERROR_ACTIONSETS_ALREADY_ATTACHED;
			}
			InteractionProfileBinding->Bindings.Emplace(Binding.action, Binding.binding, ButtonSource);

			// Update TopLevelPathInteractionProfileMap as needed
			const XrPath& TopLevelPath = InputPathToTopLevelPathMap.FindChecked(Binding.binding);
			const XrPath& CurrentTopLevelPathInteractionProfile = TopLevelPathInteractionProfileMap.FindChecked(TopLevelPath);

			if (IsFirstInputProfilePreferred(SuggestedBindings->interactionProfile, CurrentTopLevelPathInteractionProfile))
			{
				UE_LOGF(LogOXRVisionOS, Log, "TopLevelPath %ls set to InteractionProfile %ls", ANSI_TO_TCHAR(PathToString(TopLevelPath)), ANSI_TO_TCHAR(PathToString(SuggestedBindings->interactionProfile)));
				TopLevelPathInteractionProfileMap[TopLevelPath] = SuggestedBindings->interactionProfile;
			}
		}
	}

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSInstance::XrPollEvent(
	XrEventDataBuffer* EventData)
{
	// Get the next event buffer
	TSharedPtr<IEventDataHolder, ESPMode::ThreadSafe> EventHolder;
	if (EventQueue.Dequeue(EventHolder) == false)
	{
		return XR_EVENT_UNAVAILABLE;
	}

	EventHolder->CopyInto(EventData);
	return XR_SUCCESS;
}
