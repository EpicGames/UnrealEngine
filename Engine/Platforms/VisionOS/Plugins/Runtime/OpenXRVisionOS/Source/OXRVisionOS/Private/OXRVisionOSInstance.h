// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "OXRVisionOS.h"
#include "Containers/Queue.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"

class FOXRVisionOS;
class FOXRVisionOSSession;
class FOXRVisionOSActionSet;

enum class EOXRVisionOSControllerButton : int32;

class FOXRVisionOSInstance
{
public:
	static XrResult Create(TSharedPtr<FOXRVisionOSInstance, ESPMode::ThreadSafe>& OutInstance, const XrInstanceCreateInfo* createInfo, FOXRVisionOS* InModule);
	FOXRVisionOSInstance(const XrInstanceCreateInfo* createInfo, FOXRVisionOS* Module);

	~FOXRVisionOSInstance();

	XrResult XrGetInstanceProperties(
		XrInstanceProperties*						instanceProperties);

	XrResult XrGetSystemProperties(
		XrSystemId                                  systemId,
		XrSystemProperties* properties);

	XrResult XrEnumerateEnvironmentBlendModes(
		XrSystemId                                  systemId,
		XrViewConfigurationType                     viewConfigurationType,
		uint32_t                                    environmentBlendModeCapacityInput,
		uint32_t*									environmentBlendModeCountOutput,
		XrEnvironmentBlendMode*						environmentBlendModes);

	XrResult XrEnumerateViewConfigurations(
		XrSystemId                                  systemId,
		uint32_t                                    viewConfigurationTypeCapacityInput,
		uint32_t*									viewConfigurationTypeCountOutput,
		XrViewConfigurationType*					viewConfigurationTypes);

	XrResult XrGetViewConfigurationProperties(
		XrSystemId                                  systemId,
		XrViewConfigurationType                     viewConfigurationType,
		XrViewConfigurationProperties*				configurationProperties);

	XrResult XrEnumerateViewConfigurationViews(
		XrSystemId                                  systemId,
		XrViewConfigurationType                     viewConfigurationType,
		uint32_t                                    viewCapacityInput,
		uint32_t* viewCountOutput,
		XrViewConfigurationView* views);

	XrResult XrGetSystem(
		const XrSystemGetInfo*						getInfo,
		XrSystemId*									systemId);

	bool IsHMDAvailable();

	XrResult XrCreateSession(
		const XrSessionCreateInfo*					createInfo,
		XrSession*									session);

	XrResult DestroySession() { Session = nullptr; return XR_SUCCESS; }

	static bool CheckSession(XrSession session) { return (session != XR_NULL_HANDLE) && (session == (XrSession)Session.Get()); }

	XrResult XrStringToPath(
		const char* pathString,
		XrPath* path);

	XrResult XrPathToString(
		XrPath                                      path,
		uint32_t                                    bufferCapacityInput,
		uint32_t*									bufferCountOutput,
		char*										buffer);

	char* PathToString(XrPath Path) const
	{
		return XrPaths[Path].Get();
	}

	XrResult XrCreateActionSet(
		const XrActionSetCreateInfo*				createInfo,
		XrActionSet*								actionSet);

	XrResult DestroyActionSet(FOXRVisionOSActionSet* ActionSet);

	XrResult XrSuggestInteractionProfileBindings(
		const XrInteractionProfileSuggestedBinding* suggestedBindings);

	XrResult XrPollEvent(
		XrEventDataBuffer*							eventData);

	template <typename T1>
	void EnqueueEvent(T1& InEvent)
	{
		TSharedPtr<IEventDataHolder, ESPMode::ThreadSafe> DataHolderPtr(new FEventDataHolder<T1>(InEvent));
		EventQueue.Enqueue(DataHolderPtr);
	}

	bool IsExtensionEnabled(const ANSICHAR* ExtensionName) const { return EnabledExtensions.Contains(ExtensionName); } 

private:
	bool bCreateFailed = false;
	FOXRVisionOS* Module = nullptr;
	static TSharedPtr<FOXRVisionOSSession, ESPMode::ThreadSafe> Session;

	void OnWorldTickStart(UWorld* World, ELevelTick TickType, float DeltaTime);
	FDelegateHandle OnWorldTickStartDelegateHandle;
	bool bHMDWorn = false;

	struct FAnsiKeyFunc : BaseKeyFuncs<const ANSICHAR*, const ANSICHAR*, false>
	{
		typedef typename TTypeTraits<const ANSICHAR*>::ConstPointerType KeyInitType;
		typedef typename TCallTraits<const ANSICHAR*>::ParamType ElementInitType;

		/**
		 * @return The key used to index the given element.
		 */
		static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element;
		}

		/**
		 * @return True if the keys match.
		 */
		static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
		{
			return TCString<ANSICHAR>::Stricmp(A, B) == 0;
		}

		/** Calculates a hash index for a key. */
		static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
		{
			return FCrc::Strihash_DEPRECATED(Key);
		}
	};
	TSet<const ANSICHAR*, FAnsiKeyFunc> EnabledExtensions;

	XrInstanceProperties InstanceProperties;
	XrSystemProperties SystemProperties;
//	XrSystemEyeGazeInteractionPropertiesEXT EyeGazeInteractionSystemProperties;

	struct IEventDataHolder
	{
		virtual ~IEventDataHolder() {}
		virtual void CopyInto(XrEventDataBuffer* eventData) = 0;
	};

	template <typename T1>
	struct FEventDataHolder : public IEventDataHolder
	{
		FEventDataHolder(T1 InEvent)	{ Event = InEvent; check(Event.next == nullptr); }
		virtual ~FEventDataHolder()		{};
		virtual void CopyInto(XrEventDataBuffer* EventData) override
		{
			check(sizeof(T1) <= sizeof(XrEventDataBuffer))
			*reinterpret_cast<T1*>(EventData) = Event;
		}
	private:
		T1 Event;
	};

	TQueue<TSharedPtr<IEventDataHolder, ESPMode::ThreadSafe>, EQueueMode::Mpsc> EventQueue;

	XrPath StringToPath(const char* pathString);
	XrPath StringToPath(const char* pathString, uint32 PathSize);
	TArray<TUniquePtr<ANSICHAR[]>> XrPaths; //An XrPath is an index into this array
	struct FStringToXrPathKeyFunc
	{
		typedef const ANSICHAR* KeyType;
		typedef const ANSICHAR* KeyInitType;
		typedef const TPairInitializer<const ANSICHAR*, XrPath>& ElementInitType;

		enum { bAllowDuplicateKeys = false };

		static FORCEINLINE bool Matches(const ANSICHAR* A, const ANSICHAR* B)
		{
			return TCString<ANSICHAR>::Stricmp(A, B) == 0;
		}

		static FORCEINLINE uint32 GetKeyHash(const ANSICHAR* Key)
		{
			return FCrc::Strihash_DEPRECATED(Key);
		}

		static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
		{
			return Element.Key;
		}
	};
	TMap<const ANSICHAR*, XrPath, FDefaultSetAllocator, FStringToXrPathKeyFunc> StringToXrPathMap;
	TArray<TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>> ActionSets;
public:
	TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe> GetActionSetSharedPtr(XrActionSet ActionSet);
private:


	// Interaction profiles define a set of 'input paths' and how they should map to runtime hardware, like controllers.  
	// We have a set of source buttons that are 1:1 with specific hardware.
	// The OXRVisionOS native interaction profile, therefore, simply maps every path to its button.
	// Other profiles will map their input paths to one button.
	// Interaction profiles will, currently, never bind multiple source buttons to a single profile path.
	TMap<XrPath, TMap<XrPath, EOXRVisionOSControllerButton>> InteractionProfiles; // Profile, input path, native path.
	void CreateInteractionProfiles();

	// The runtime must select one interaction profile for each top level path from among those the application attempts to bind.
	bool IsFirstInputProfilePreferred(XrPath First, XrPath Second) const;
	TMap<XrPath, XrPath> TopLevelPathInteractionProfileMap; // Map of top level paths to most preferred bound input profile
	TMap<XrPath, XrPath> InputPathToTopLevelPathMap; // Map of input paths to Top level paths (eg /user/hand/left)

	// XrSuggestInteractionProfileBindings takes a set of action-binding mappings and one of the InteractionProfiles we define and generates
	// an array of bindings for that profile.  An action can have multiple suggested paths.
	// See XrAttachSessionActionSets for the next step in the input process.
public:
	struct FInteractionProfileBinding
	{
		XrPath InteractionProfilePath = XR_NULL_PATH;
		struct FBinding
		{
			FBinding()
			{}
			FBinding(XrAction InAction, XrPath InPath, EOXRVisionOSControllerButton InButton)
				: Action(InAction)
				, Path(InPath)
				, Button(InButton)
			{}
			XrAction Action;
			XrPath Path = XR_NULL_PATH;
			EOXRVisionOSControllerButton Button;
		};
		TArray<FBinding> Bindings;
	};
	TMap<XrPath, FInteractionProfileBinding> InteractionProfileBindings;
	TArray<XrPath> PreferredInteractionProfileBindingList; // The list of supported interaction profiles ordered from most to least preferred.
	const TMap<XrPath, FInteractionProfileBinding>& GetInteractionProfileBindings() const {	return InteractionProfileBindings; }
	const TArray<XrPath>& GetPreferredInteractionProfileBindingList() const { return PreferredInteractionProfileBindingList; }
	XrPath GetCurrentInteractionProfile(XrPath TopLevelUserPath) const { return TopLevelPathInteractionProfileMap.FindChecked(TopLevelUserPath); }
	XrPath InputPathToTopLevelPath(XrPath InPath) const { return InputPathToTopLevelPathMap.Contains(InPath) ? InputPathToTopLevelPathMap.FindChecked(InPath) : XR_NULL_PATH; }
	XrPath TopLevelPathToInteractionProfile(XrPath InPath) const { return TopLevelPathInteractionProfileMap.Contains(InPath) ? TopLevelPathInteractionProfileMap.FindChecked(InPath) : XR_NULL_PATH; }
};
