// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define XR_USE_GRAPHICS_API_OXRVISIONOS

#ifdef XR_USE_GRAPHICS_API_OXRVISIONOS
#include "DynamicRHI.h"
#endif // XR_USE_GRAPHICS_API_OXRVISIONOS

#include <openxr/openxr_platform.h>

