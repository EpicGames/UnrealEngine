// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OpenXRCore.h"
#include "Epic_openxr.h"
#import <simd/simd.h>

// OXRVisionOS works with UE4, OpenXR, and VisionOS types.  OpenXR and VisionOS coordinate systems are the same.

namespace OXRVisionOS
{
	struct TrackerPoseData
	{
		simd_float3 position = {0.0f,0.0f,0.0f};
		simd_float4 orientation = {0.0f,0.0f,0.0f,1.0f};
	};

	struct TrackerResultData
	{
		TrackerPoseData Pose;
	};


	const XrVector3f XrZeroVector = {};

	FORCEINLINE FVector ToFVector(const simd_float3& InVector)
	{
		return FVector(
			-InVector.z,
			InVector.x,
			InVector.y
		);
	}
	FORCEINLINE simd_float3 ToSimdFVector3(const FVector& InVector)
	{
		return simd_float3{
			static_cast<float>(InVector.Y),
			static_cast<float>(InVector.Z),
			static_cast<float>(-InVector.X)
		};
	}
	FORCEINLINE simd_float3 ToSimdFVector3(const XrVector3f& InVector)
	{
		return simd_float3{
			InVector.x,
			InVector.y,
			InVector.z
		};
	}
	FORCEINLINE XrVector3f ToXrVector3f(const FVector& InVector)
	{
		return XrVector3f{
			static_cast<float>(InVector.Y),
			static_cast<float>(InVector.Z),
			static_cast<float>(-InVector.X)
		};
	}

	FORCEINLINE XrVector3f ToXrVector3f(const simd_float3& InVector)
	{
		return XrVector3f{
			InVector.x,
			InVector.y,
			InVector.z
		};
	}

	FORCEINLINE FQuat ToFQuat(const simd_float4& InQuaternion)
	{
		return FQuat(
			-InQuaternion.z,
			InQuaternion.x,
			InQuaternion.y,
			-InQuaternion.w
		);
	}
	FORCEINLINE simd_float4 ToSimdFQuaternion(const FQuat& InQuaternion)
	{
		return simd_float4{
			static_cast<float>(InQuaternion.Y),
			static_cast<float>(InQuaternion.Z),
			static_cast<float>(-InQuaternion.X),
			static_cast<float>(-InQuaternion.W)
		};
	}
	FORCEINLINE simd_float4 ToSimdFQuaternion(const XrQuaternionf& InQuaternion)
	{
		return simd_float4{
			InQuaternion.x,
			InQuaternion.y,
			InQuaternion.z,
			InQuaternion.w
		};
	}
	FORCEINLINE XrQuaternionf ToXrQuaternionf(const FQuat& InQuaternion)
	{
		return XrQuaternionf{
			static_cast<float>(InQuaternion.Y),
			static_cast<float>(InQuaternion.Z),
			static_cast<float>(-InQuaternion.X),
			static_cast<float>(-InQuaternion.W)
		};
	}

    /**
     * Convert's an ARKit 'Y up' 'right handed' coordinate system transform to Unreal's 'Z up'
     * 'left handed' coordinate system.
     *
     * Scale is removed because some ARKit transforms are less normalized than unreal requires.
     */
    static FORCEINLINE FTransform ToFTransform(const matrix_float4x4& RawYUpMatrix)
    {
        // Conversion here is as per SteamVRHMD::ToFMatrix
        FMatrix RawYUpFMatrix(
            FPlane(RawYUpMatrix.columns[0][0], RawYUpMatrix.columns[0][1], RawYUpMatrix.columns[0][2], RawYUpMatrix.columns[0][3]),
            FPlane(RawYUpMatrix.columns[1][0], RawYUpMatrix.columns[1][1], RawYUpMatrix.columns[1][2], RawYUpMatrix.columns[1][3]),
            FPlane(RawYUpMatrix.columns[2][0], RawYUpMatrix.columns[2][1], RawYUpMatrix.columns[2][2], RawYUpMatrix.columns[2][3]),
            FPlane(RawYUpMatrix.columns[3][0], RawYUpMatrix.columns[3][1], RawYUpMatrix.columns[3][2], RawYUpMatrix.columns[3][3]));

        // ARKit, particularly when restoring a previously saved AR world map, sometimes has transforms who's normalization tolerance is not
        // tight enough to avoid ensures or even failure that result in identity rotations from FQuat.  So we remove scaling here.
        RawYUpFMatrix.RemoveScaling(UE_KINDA_SMALL_NUMBER / 100.0f);

        // Extract & convert translation
        FVector Translation = FVector( -RawYUpFMatrix.M[3][2], RawYUpFMatrix.M[3][0], RawYUpFMatrix.M[3][1] );

        // Extract & convert rotation
        FQuat RawRotation( RawYUpFMatrix );
        FQuat Rotation( -RawRotation.Z, RawRotation.X, RawRotation.Y, -RawRotation.W );

        return FTransform( Rotation, Translation );
    }

	FORCEINLINE FTransform ToFTransform(const TrackerPoseData& InPoseData)
	{
		return FTransform(
			ToFQuat(InPoseData.orientation),
			ToFVector(InPoseData.position)
			);
	}
	FORCEINLINE TrackerPoseData ToPoseData(const FTransform& InTransform)
	{
		TrackerPoseData PoseData;
		PoseData.orientation = ToSimdFQuaternion(InTransform.GetRotation());
		PoseData.position = ToSimdFVector3(InTransform.GetLocation());
		return PoseData;
	}

	FORCEINLINE XrPosef ToXrPose(const TrackerPoseData& InPoseData)
	{
		XrPosef Ret;
		Ret.orientation = XrQuaternionf{
			InPoseData.orientation.x,
			InPoseData.orientation.y,
			InPoseData.orientation.z,
			InPoseData.orientation.w };
		Ret.position = XrVector3f{ 
			InPoseData.position.x,
			InPoseData.position.y,
			InPoseData.position.z };
		return Ret;
	}
    FORCEINLINE XrPosef ToXrPose(const simd_float4x4& RawYUpMatrix)
    {
        // Conversion here is as per SteamVRHMD::ToFMatrix
        FMatrix RawYUpFMatrix(
            FPlane(RawYUpMatrix.columns[0][0], RawYUpMatrix.columns[0][1], RawYUpMatrix.columns[0][2], RawYUpMatrix.columns[0][3]),
            FPlane(RawYUpMatrix.columns[1][0], RawYUpMatrix.columns[1][1], RawYUpMatrix.columns[1][2], RawYUpMatrix.columns[1][3]),
            FPlane(RawYUpMatrix.columns[2][0], RawYUpMatrix.columns[2][1], RawYUpMatrix.columns[2][2], RawYUpMatrix.columns[2][3]),
            FPlane(RawYUpMatrix.columns[3][0], RawYUpMatrix.columns[3][1], RawYUpMatrix.columns[3][2], RawYUpMatrix.columns[3][3]));

        // ARKit, particularly when restoring a previously saved AR world map, sometimes has transforms who's normalization tolerance is not
        // tight enough to avoid ensures or even failure that result in identity rotations from FQuat.  So we remove scaling here.
        RawYUpFMatrix.RemoveScaling(UE_KINDA_SMALL_NUMBER / 100.0f);

        // Extract rotation
        const FQuat RawRotation( RawYUpFMatrix );

        XrPosef Ret;
        Ret.orientation.x = RawRotation.X;
        Ret.orientation.y = RawRotation.Y;
        Ret.orientation.z = RawRotation.Z;
        Ret.orientation.w = RawRotation.W;

		Ret.position.x = RawYUpMatrix.columns[3][0];
		Ret.position.y = RawYUpMatrix.columns[3][1];
		Ret.position.z = RawYUpMatrix.columns[3][2];
		
		return Ret;
    }
	FORCEINLINE XrPosef ToXrPose(const FTransform& InTransform)
	{
		XrPosef Ret;
		Ret.orientation = ToXrQuaternionf(InTransform.GetRotation());
		Ret.position = ToXrVector3f(InTransform.GetLocation());
		return Ret;
	}

	FORCEINLINE XrSpaceLocationFlags ToXrSpaceLocationFlags(TrackerResultData& InResultData)
	{
//TODO finish this
		XrSpaceLocationFlags OutLocationFlags = 0;
		 	OutLocationFlags |= XR_SPACE_LOCATION_POSITION_VALID_BIT;
		// 	OutLocationFlags |= (XR_SPACE_LOCATION_POSITION_VALID_BIT | XR_SPACE_LOCATION_POSITION_TRACKED_BIT);


		 	OutLocationFlags |= XR_SPACE_LOCATION_ORIENTATION_VALID_BIT;
		// 	OutLocationFlags |= (XR_SPACE_LOCATION_ORIENTATION_VALID_BIT | XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT);


		return OutLocationFlags;
	}

	FORCEINLINE XrSpaceVelocityFlags ToXrSpaceVelocityFlags(TrackerResultData& InResultData)
	{
//TODO finish this, or delete it
		XrSpaceVelocityFlags OutFlags = 0;

		// 	OutFlags |= XR_SPACE_VELOCITY_LINEAR_VALID_BIT;
		// 	OutFlags |= XR_SPACE_VELOCITY_ANGULAR_VALID_BIT;

		return OutFlags;
	}
	

	// CFTimeInterval is a double of seconds.  XrTime is uint64 nanoseconds.
	FORCEINLINE XrTime APITimeToXrTime(CFTimeInterval TimeInterval)
	{
		return (XrTime)(TimeInterval * 1000000000.0);
	}

	FORCEINLINE CFTimeInterval XrTimeToAPITime(XrTime xrTime)
	{
		return xrTime / 1000000000.0;
	}
};
