// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSSession.h"
#include "OXRVisionOSAction.h"
#include "OXRVisionOSActionSet.h"
#include "OXRVisionOSController.h"
#include "OXRVisionOSInstance.h"
#include "OXRVisionOSPlatformUtils.h"
#include "OXRVisionOS_RenderBridge.h"
#include "OXRVisionOSRuntimeSettings.h"
#include "OXRVisionOSSettingsTypes.h"
#include "OXRVisionOSSpace.h"
#include "OXRVisionOSSwapchain.h"
#include "OXRVisionOSTracker.h"
#include "OXRVisionOS_openxr_platform.h"
#include "OXRVisionOSAppDelegate.h"
#include "PixelFormat.h"
#include "StereoRenderTargetManager.h"
#include "HAL/RunnableThread.h"

#include "RenderingThread.h"

#include "CoreGlobals.h"
#include "RHICommandList.h"
#include "MetalRHIVisionOSBridge.h"

#define CHECK_API_RET_CREATE_FAIL(___function) {int __ret = RESULT_OK; __ret = (___function); if(__ret < RESULT_OK) { UE_LOG(LogOXRVisionOS, Warning, TEXT("Call to " #___function " failed, returned %d 0x%x"), __ret, __ret); UE_DEBUG_BREAK(); bCreateFailed = true; return; }}
#define CHECK_API_RET_RETURN_FAIL(___function) {int __ret = RESULT_OK; __ret = (___function); if(__ret < RESULT_OK) { UE_LOG(LogOXRVisionOS, Warning, TEXT("Call to " #___function " failed, returned %d 0x%x"), __ret, __ret); UE_DEBUG_BREAK(); return XrResult::XR_ERROR_RUNTIME_FAILURE; }}
#define CHECK_API_RET_WARN(___function) {int __ret = RESULT_OK; __ret = (___function); if(__ret < RESULT_OK) { UE_LOG(LogOXRVisionOS, Warning, TEXT("Call to " #___function " failed, returned %d 0x%x"), __ret, __ret);}}

#define OXRVISIONOS_REPROJECTION_ANALYSIS 0

// Helper macro for checking Api error codes. If the result of a function call is negative, causes a fatal log.
#define FA_CHECK_F(code, format, ...) do { int32 Result; { Result = (code); } if (Result < 0) { UE_LOG(LogOXRVisionOS, Fatal, TEXT("%s failed with error code: 0x%08x") format, TEXT(#code), Result, ##__VA_ARGS__); } } while(false)
#define FA_CHECK(code) FA_CHECK_F(code, TEXT(""))

namespace OXRVisionOSCVars
{
	static TAutoConsoleVariable<bool> CVarDisableReprojection(
		TEXT("vr.OXRVisionOS.DisableReprojection"),
		false,
		TEXT("Disables reprojection for debugging purposes"));
}

namespace OXRVisionOSSessionHelpers
{
	enum ELayerIndex : uint8
	{
		Background = 0,
		//CompositedOverlays,
		Num
	};

	constexpr uint32 OXRVisionOSBackbufferLength = 2;
	constexpr uint32 MaxOXRVisionOSLayers = 1; //TODO for now I only want the one stereo layer, 
}

XrResult FOXRVisionOSSession::Create(TSharedPtr<FOXRVisionOSSession, ESPMode::ThreadSafe>& OutSession, const XrSessionCreateInfo* createInfo, FOXRVisionOSInstance* Instance)
{
	if (createInfo == nullptr || Instance == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (createInfo->type != XR_TYPE_SESSION_CREATE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	const XrGraphicsBindingOXRVisionOSEPIC* GraphicsBinding = OpenXR::FindChainedStructByType<XrGraphicsBindingOXRVisionOSEPIC>(createInfo->next, (XrStructureType)XR_TYPE_GRAPHICS_BINDING_OXRVISIONOS_EPIC);
	if (GraphicsBinding)
	{
		//TODO use the graphics binding?
	}

	OutSession = MakeShared<FOXRVisionOSSession, ESPMode::ThreadSafe>(createInfo, Instance);
	if (OutSession->bCreateFailed)
	{
		OutSession = nullptr;
		return XrResult::XR_ERROR_RUNTIME_FAILURE;
	}
	return XrResult::XR_SUCCESS;
}


FOXRVisionOSSession::FOXRVisionOSSession(const XrSessionCreateInfo* createInfo, FOXRVisionOSInstance* InInstance)
{
	Instance = InInstance;

	if (FOXRVisionOSTracker::Create(Tracker, Instance->IsExtensionEnabled(XR_EXT_EYE_GAZE_INTERACTION_EXTENSION_NAME)) == false)
	{
		UE_LOGF(LogOXRVisionOS, Warning, "FOXRVisionOSTracker::Create failed.  See logging above."); 
		bCreateFailed = true;
		return;
	}

	//if (FOXRVisionOSController::Create(Controllers, this) == false)
	//{
	//	UE_LOGF(LogOXRVisionOS, Warning, "FOXRVisionOSController::Create failed.  See logging above.");
	//	bCreateFailed = true;
	//	return;
	//}

	const UOXRVisionOSRuntimeSettings* Settings = GetDefault<UOXRVisionOSRuntimeSettings>();
	AimPoseAdjustment = FQuat(Settings->OXRVisionOSAimPoseAdjustment);
	GripPoseAdjustment = FQuat(Settings->OXRVisionOSGripPoseAdjustment);
	GripSurfacePoseAdjustment = FQuat(Settings->OXRVisionOSGripSurfacePoseAdjustment);

	static_assert(OXRVisionOSSessionHelpers::ELayerIndex::Num <= OXRVisionOSSessionHelpers::MaxOXRVisionOSLayers);

	RenderBridge = FOXRVisionOS::Get()->GetRenderBridge();
	check(RenderBridge != nullptr);

	ARKitSession = ar_session_create();

	XrWaitFrameEvent = FPlatformProcess::GetSynchEventFromPool(false);
	XrWaitFrameEvent->Trigger();

	SetSessionState(XR_SESSION_STATE_IDLE);

	// We need to wait for the layerstate to transition before starting.
	OnWorldTickStartDelegateHandle = FWorldDelegates::OnWorldTickStart.AddRaw(this, &FOXRVisionOSSession::OnWorldTickStart);
}

void FOXRVisionOSSession::OnWorldTickStart(UWorld* World, ELevelTick TickType, float DeltaTime)
{
	if (SessionState == XR_SESSION_STATE_IDLE)
	{
		FScopeLock Lock(&CpLayerMutex);
		switch (cp_layer_renderer_get_state(OXRVisionOS::GetSwiftLayerRenderer()))
		{
			case cp_layer_renderer_state_paused:
				// Wait until the scene appears.
				// Polling instead of using this wait function.
				//cp_layer_renderer_wait_until_running(layer);
				UE_LOGF(LogOXRVisionOS, Warning, "FOXRVisionOSSession cp_layer_renderer_get_state is paused.  Waiting in XR_SESSION_STATE_IDLE.");
				break;

			case cp_layer_renderer_state_running:
				UE_LOGF(LogOXRVisionOS, Warning, "FOXRVisionOSSession cp_layer_renderer_get_state is running. Going to XR_SESSION_STATE_READY");

				SetSessionState(XR_SESSION_STATE_READY);
				break;

			case cp_layer_renderer_state_invalidated:
				// Not entirely sure this is correct, but it's kind of close.
				UE_LOGF(LogOXRVisionOS, Warning, "FOXRVisionOSSession cp_layer_renderer_get_state is cp_layer_renderer_state_invalidated.  Ending session.");
				EndSessionInternal();
				break;
				
			default:
				check(false);
		 }
	}

	// Stop trying to transition to READY when no longer IDLE.
	if (SessionState != XR_SESSION_STATE_IDLE)
	{
		FWorldDelegates::OnWorldTickStart.Remove(OnWorldTickStartDelegateHandle);
	}
}

FOXRVisionOSSession::~FOXRVisionOSSession()
{
	FWorldDelegates::OnWorldTickStart.Remove(OnWorldTickStartDelegateHandle);

	//XrDestroySession can be called from any session state, so all of them must cleanup correctly.
	if (IsSessionRunning())
	{
		EndSessionInternal();
	}

	Controllers = nullptr;
	Tracker = nullptr;

	FPlatformProcess::ReturnSynchEventToPool(XrWaitFrameEvent);
	XrWaitFrameEvent = nullptr;

	if (bCreateFailed)
	{
		UE_LOGF(LogOXRVisionOS, Warning, "Destructing FOXRVisionOSSession because session create failed.  You may see more warnings as api shutdown calls fail, depending on how the create failed.");
	}

	{
		// We need to unregister the swapchain buffers from the video out.  
		// First we have to make sure they aren't being used by flipping to blank and flushing it through.
		// Then we can unregister.

		// Completely drain all remaining GPU work	
		ENQUEUE_RENDER_COMMAND(FlipToBlank)([this](FRHICommandListImmediate& RHICmdList)
			{
				// Flip to blank and drain the GPU
				RHICmdList.EnqueueLambda([this](FRHICommandListImmediate& RHICmdList)
					{
						//TODO
						//MetalRHIOXRVisionOS::SetFlipToBlankAndFinalize(RHICmdList);

						//// Wait for all outstanding flips to clear, including our flip to blank.
					});

				// Perform a full flush
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

				RHICmdList.SubmitAndBlockUntilGPUIdle();
			});

		FlushRenderingCommands();
	}
}

XrResult FOXRVisionOSSession::XrDestroySession()
{
	// Instance->DestroySession() can delete this, so better just return after that.
	return Instance->DestroySession();
}

XrResult FOXRVisionOSSession::XrEnumerateReferenceSpaces(
	uint32_t                                    SpaceCapacityInput,
	uint32_t*									SpaceCountOutput,
	XrReferenceSpaceType*						OutSpaces)
{
	if (SpaceCountOutput == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	if (SpaceCapacityInput != 0 && OutSpaces == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	const int NumSupportedSpaceTypes = 3;//4;

	*SpaceCountOutput = NumSupportedSpaceTypes;
	if (SpaceCapacityInput != 0)
	{
		if (SpaceCapacityInput < *SpaceCountOutput)
		{
			return XrResult::XR_ERROR_SIZE_INSUFFICIENT;
		}

		OutSpaces[0] = XR_REFERENCE_SPACE_TYPE_VIEW;
		OutSpaces[1] = XR_REFERENCE_SPACE_TYPE_LOCAL;
		OutSpaces[2] = XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR;
		//OutSpaces[3] = XR_REFERENCE_SPACE_TYPE_STAGE;
	}
	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrCreateReferenceSpace(
	const XrReferenceSpaceCreateInfo*			CreateInfo,
	XrSpace*									Space)
{
	if (!CreateInfo || CreateInfo->type != XR_TYPE_REFERENCE_SPACE_CREATE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe> NewSpace = nullptr;
	XrResult Ret = FOXRVisionOSSpace::CreateReferenceSpace(NewSpace, CreateInfo, this);
	if (Ret == XrResult::XR_SUCCESS)
	{
		Spaces.Add(NewSpace);
		*Space = (XrSpace)NewSpace.Get();
	}

	return Ret;
}

XrResult FOXRVisionOSSession::XrGetReferenceSpaceBoundsRect(
	XrReferenceSpaceType                    ReferenceSpaceType,
	XrExtent2Df*							Bounds)
{
	if (!Bounds)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (ReferenceSpaceType != XR_REFERENCE_SPACE_TYPE_STAGE)
	{
		UE_LOGF(LogOXRVisionOS, Log, "XrGetReferenceSpaceBoundsRect called with an reference space type that does not support it");
		Bounds->width = 0.0f;
		Bounds->height = 0.0f;
		return XR_SPACE_BOUNDS_UNAVAILABLE;
	}

	// According to docs the play area is a 1.5m sphere around the hmd.
	// We will not return any bounds for now.
	return XR_SPACE_BOUNDS_UNAVAILABLE;

	// Bounds->width = ;
	// Bounds->height = ;
	// return XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrCreateActionSpace(
	const XrActionSpaceCreateInfo*	CreateInfo,
	XrSpace*						Space)
{
	if (!CreateInfo || CreateInfo->type != XR_TYPE_ACTION_SPACE_CREATE_INFO || !Space)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe> NewSpace = nullptr;
	XrResult Ret = FOXRVisionOSSpace::CreateActionSpace(NewSpace, CreateInfo, this);
	if (Ret == XrResult::XR_SUCCESS)
	{
		Spaces.Add(NewSpace);
		*Space = (XrSpace)NewSpace.Get();
	}
	return Ret;
}

const FOXRVisionOSSpace* FOXRVisionOSSession::ToFOXRVisionOSSpace(const XrSpace& Space)
{
	// Check that this is a space we know about.
	uint32 ArrayIndex = Spaces.IndexOfByPredicate([Space](const TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe>& Data) { return (Data.Get() == (FOXRVisionOSSpace*)Space); });
	if (ArrayIndex == INDEX_NONE)
	{
		return nullptr;
	}
	
	return (FOXRVisionOSSpace*)Space;
}

XrResult FOXRVisionOSSession::DestroySpace(FOXRVisionOSSpace* Space)
{
	uint32 ArrayIndex = Spaces.IndexOfByPredicate([Space](const TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe>& Data) { return (Data.Get() == Space); });

	if (ArrayIndex == INDEX_NONE)
	{
		return  XrResult::XR_ERROR_HANDLE_INVALID;
	}

	Spaces.RemoveAtSwap(ArrayIndex);

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrEnumerateSwapchainFormats(
	uint32_t                                    FormatCapacityInput,
	uint32_t*									FormatCountOutput,
	int64_t*									Formats)
{
	if (FormatCountOutput == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	if (FormatCapacityInput != 0 && Formats == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	const int NumSupportedFormats = 4;

	*FormatCountOutput = NumSupportedFormats;
	if (FormatCapacityInput != 0)
	{
		if (FormatCapacityInput < *FormatCountOutput)
		{
			return XrResult::XR_ERROR_SIZE_INSUFFICIENT;
		}

		// HDR formats 
		Formats[0] = PF_B8G8R8A8;			//bgra8Unorm_srgb			// This is Unreal's default format
		Formats[1] = PF_FloatRGBA; 			//rgba16Float

		//Depth formats
		Formats[2] = PF_DepthStencil;		//depth32Float_stencil8		// Correct for deferred
		Formats[3] = PF_R32_FLOAT;			//depth32Float				// Possibly correct for mobile forward
	}
	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrCreateSwapchain(
	const XrSwapchainCreateInfo*				CreateInfo,
	XrSwapchain*								Swapchain)
{
	if (CreateInfo == nullptr || CreateInfo->type != XR_TYPE_SWAPCHAIN_CREATE_INFO || Swapchain == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	TSharedPtr<FOXRVisionOSSwapchain, ESPMode::ThreadSafe> NewSwapchain = nullptr;
	XrResult Ret = FOXRVisionOSSwapchain::Create(NewSwapchain, CreateInfo, OXRVisionOSSessionHelpers::OXRVisionOSBackbufferLength, this);
	if (Ret == XrResult::XR_SUCCESS)
	{
		*Swapchain = (XrSwapchain)NewSwapchain.Get();
		Swapchains.Add(NewSwapchain);
	}
	return Ret;
}

XrResult FOXRVisionOSSession::DestroySwapchain(
	FOXRVisionOSSwapchain* Swapchain)
{
	uint32 ArrayIndex = Swapchains.IndexOfByPredicate([Swapchain](const TSharedPtr<FOXRVisionOSSwapchain, ESPMode::ThreadSafe>& Data) { return (Data.Get() == Swapchain); });

	if (ArrayIndex == INDEX_NONE)
	{
		return XrResult::XR_ERROR_HANDLE_INVALID;
	}

	Swapchains.RemoveAtSwap(ArrayIndex);

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrAttachSessionActionSets(
	const XrSessionActionSetsAttachInfo* AttachInfo)
{
	if (AttachInfo == nullptr || AttachInfo->type != XR_TYPE_SESSION_ACTION_SETS_ATTACH_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (ActionSetsAlreadyAttached)
	{
		return XrResult::XR_ERROR_ACTIONSETS_ALREADY_ATTACHED;
	}

	UE_LOGF(LogOXRVisionOS, Verbose, "XrAttachSessionActionSets Attaching");

	check(AttachInfo->next == nullptr);
	for (int i = 0; i < AttachInfo->countActionSets; i++)
	{
		XrActionSet ActionSet = AttachInfo->actionSets[i];
		check(ActionSet != XR_NULL_HANDLE);
		TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe> ActionSetPtr = Instance->GetActionSetSharedPtr(ActionSet);
		ActionSetPtr->Attach(this);
		ActionSets.Add(ActionSetPtr); 
	}

	// Choose the InteractionProfileBinding we will actually use.
	bool bBoundAnInteractionProfile = false;
	for (auto& InteractionProfilePath : Instance->GetPreferredInteractionProfileBindingList())
	{
		const FOXRVisionOSInstance::FInteractionProfileBinding* IPB = Instance->GetInteractionProfileBindings().Find(InteractionProfilePath);

		if (IPB != nullptr)
		{
			UE_LOGF(LogOXRVisionOS, Verbose, "XrAttachSessionActionSets found interaction profile %ls and will bind inputs using it.", ANSI_TO_TCHAR(Instance->PathToString(IPB->InteractionProfilePath)));
			bBoundAnInteractionProfile = true;

			for (TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>& ActionSet : ActionSets)
			{
				// Each action may be bound to one or more source
				for (TSharedPtr<FOXRVisionOSAction, ESPMode::ThreadSafe> ActionSP : ActionSet->GetActions())
				{
					FOXRVisionOSAction* Action = ActionSP.Get();
					for (const FOXRVisionOSInstance::FInteractionProfileBinding::FBinding& Binding : IPB->Bindings)
					{
						if (Binding.Action == (XrAction)Action)
						{
							// If this interaction profile is the current interaction profile of the top level path of this binding bind it.
							XrPath TopLevelBindingPath = Instance->InputPathToTopLevelPath(Binding.Path);
							XrPath TopLevelPathInteractionProfile = Instance->TopLevelPathToInteractionProfile(TopLevelBindingPath);
							if (InteractionProfilePath == TopLevelPathInteractionProfile)
							{
								UE_LOGF(LogOXRVisionOS, Verbose, "  Binding %ls to %ls", ANSI_TO_TCHAR(Instance->PathToString(Binding.Path)), OXRVisionOS::OXRVisionOSControllerButtonToTCHAR(Binding.Button));
								Action->BindSource(Binding.Path, Binding.Button);
							}
							else
							{
								UE_LOGF(LogOXRVisionOS, Verbose, "  NOT binding %ls to %ls because this is not the best interaction profile", ANSI_TO_TCHAR(Instance->PathToString(Binding.Path)), OXRVisionOS::OXRVisionOSControllerButtonToTCHAR(Binding.Button));
							}
						}
					}
				}
			}
		}
	}
	
	if (!bBoundAnInteractionProfile)
	{
		UE_LOGF(LogOXRVisionOS, Warning, "XrAttachSessionActionSets has been called, but no supported interaction profiles were suggested through xrSuggestInteractionProfileBindings!  None of the input actions will function.");
	}
	
	ActionSetsAlreadyAttached = true;
	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrGetCurrentInteractionProfile(
	XrPath TopLevelUserPath,
	XrInteractionProfileState* InteractionProfile)
{
	if (ActionSetsAlreadyAttached == false)
	{
		return XrResult::XR_ERROR_ACTIONSET_NOT_ATTACHED;
	}

	InteractionProfile->interactionProfile = Instance->GetCurrentInteractionProfile(TopLevelUserPath);

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrGetActionStateBoolean(
	const XrActionStateGetInfo* GetInfo,
	XrActionStateBoolean* State)
{
	const FOXRVisionOSAction* Action = (const FOXRVisionOSAction*)GetInfo->action;
	if (Action)
	{
		return Action->GetActionStateBoolean(GetInfo->subactionPath, *State);
	}
	else
	{
		return XR_ERROR_HANDLE_INVALID;
	}
}

XrResult FOXRVisionOSSession::XrGetActionStateFloat(
	const XrActionStateGetInfo* GetInfo,
	XrActionStateFloat* State)
{
	const FOXRVisionOSAction* Action = (const FOXRVisionOSAction*)GetInfo->action;
	if (Action)
	{
		return Action->GetActionStateFloat(GetInfo->subactionPath, *State);
	}
	else
	{
		return XR_ERROR_HANDLE_INVALID;
	}
}

XrResult FOXRVisionOSSession::XrGetActionStateVector2f(
	const XrActionStateGetInfo* GetInfo,
	XrActionStateVector2f* State)
{
	const FOXRVisionOSAction* Action = (const FOXRVisionOSAction*)GetInfo->action;
	if (Action)
	{
		return Action->GetActionStateVector2f(GetInfo->subactionPath, *State);
	}
	else
	{
		return XR_ERROR_HANDLE_INVALID;
	}
}

XrResult FOXRVisionOSSession::XrGetActionStatePose(
	const XrActionStateGetInfo* GetInfo,
	XrActionStatePose* State)
{
	const FOXRVisionOSAction* Action = (const FOXRVisionOSAction*)GetInfo->action;
	if (Action)
	{
		return Action->GetActionStatePose(GetInfo->subactionPath, *State);
	}
	else
	{
		return XR_ERROR_HANDLE_INVALID;
	}
}

XrResult FOXRVisionOSSession::XrSyncActions(
	const XrActionsSyncInfo* SyncInfo)
{
	check(SyncInfo);

	//TODO XR_SESSION_LOSS_PENDING

	if (SessionState != XR_SESSION_STATE_FOCUSED)
	{
		return XrResult::XR_SESSION_NOT_FOCUSED;
	}

	// Collect the hardware data
	//Controllers->SyncActions();
	const FPipelinedFrameState& PipelinedFrameState = GetPipelinedFrameStateForThread();
	Tracker->SyncActions(PipelinedFrameState.FrameCounter, PipelinedFrameState.PredictedDisplayTime);

	// Clear active from all action sets
	for (TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>& ActionSet : ActionSets)
	{
		ActionSet->ClearActive();
	}
	
	//TArray<FOXRVisionOSActionSet*> ActiveActionSetsByPriority; // Note ActionSetPriority is currently unsupported on OXRVisionOS.
	for (int32 i = 0; i < SyncInfo->countActiveActionSets; i++)
	{
		XrActiveActionSet ActiveXRActionSet = SyncInfo->activeActionSets[i];
		FOXRVisionOSActionSet& ActiveActionSet = *((FOXRVisionOSActionSet*)ActiveXRActionSet.actionSet);
		ActiveActionSet.SetActive(ActiveXRActionSet.subactionPath);
		//ActiveActionSetsByPriority.Add(&ActiveActionSet);
	}
	// Sort highest priority to the front.
	//ActiveActionSetsByPriority.Sort([](const FOXRVisionOSActionSet& A, const FOXRVisionOSActionSet& B) { return A.GetPriority() < B.GetPriority(); });

//UE_LOGF(LogOXRVisionOS, Log, "XrSyncActions PipelinedFrameState.PredictedDisplayTime %lld", PipelinedFrameState.PredictedDisplayTime);

	// Go through the action sets updating state per hardware.
//	for (TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>& ActionSet : ActionSets)
//	{
//		ActionSet->SyncActions(Controllers, Tracker);
//	}
	
	SyncHandTracking();

	return XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrEnumerateBoundSourcesForAction(
	const XrBoundSourcesForActionEnumerateInfo*		EnumerateInfo,
	uint32_t										SourceCapacityInput,
	uint32_t*										SourceCountOutput,
	XrPath*											OutSources)
{
	if (SourceCountOutput == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	if (SourceCapacityInput != 0 && OutSources == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (EnumerateInfo == nullptr || EnumerateInfo->action == XR_NULL_HANDLE)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	FOXRVisionOSAction* Action = (FOXRVisionOSAction*)EnumerateInfo->action;
	const TArray<XrPath>& Sources = Action->GetBoundSources();

	const int Num = Sources.Num();
	*SourceCountOutput = Num;
	if (SourceCapacityInput != 0)
	{
		if (SourceCapacityInput < *SourceCountOutput)
		{
			return XrResult::XR_ERROR_SIZE_INSUFFICIENT;
		}

		for (int32 i = 0; i < Num; ++i)
		{
			OutSources[i] = Sources[i];
		}
	}
	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrApplyHapticFeedback(
	const XrHapticActionInfo* HapticActionInfo,
	const XrHapticBaseHeader* HapticFeedback)
{
	if (SessionState != XR_SESSION_STATE_FOCUSED)
	{
		return XrResult::XR_SESSION_NOT_FOCUSED;
	}

	UE_LOGF(LogOXRVisionOS, Warning, "FOXRVisionOSSession XrApplyHapticFeedback returning XR_ERROR_PATH_UNSUPPORTED for path because there are no supported haptic paths on this platform.");
	return XrResult::XR_ERROR_PATH_UNSUPPORTED;
}

XrResult FOXRVisionOSSession::XrStopHapticFeedback(
	const XrHapticActionInfo* HapticActionInfo)
{
	if (SessionState != XR_SESSION_STATE_FOCUSED)
	{
		return XrResult::XR_SESSION_NOT_FOCUSED;
	}

	UE_LOGF(LogOXRVisionOS, Warning, "FOXRVisionOSSession XrApplyHapticFeedback returning XR_ERROR_PATH_UNSUPPORTED for path because there are no supported haptic paths on this platform.");
	return XrResult::XR_ERROR_PATH_UNSUPPORTED;
}

XrResult FOXRVisionOSSession::XrBeginSession(
	const XrSessionBeginInfo* BeginInfo)
{
	if (BeginInfo == nullptr || BeginInfo->type != XR_TYPE_SESSION_BEGIN_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (SessionState == XrSessionState::XR_SESSION_STATE_UNKNOWN  || SessionState == XrSessionState::XR_SESSION_STATE_IDLE)
	{
		return XrResult::XR_ERROR_SESSION_NOT_READY;
	}

	if (IsSessionRunning())
	{
		return XrResult::XR_ERROR_SESSION_RUNNING;
	}

	if (BeginInfo->primaryViewConfigurationType != XrViewConfigurationType::XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO)
	{
		return XrResult::XR_ERROR_VIEW_CONFIGURATION_TYPE_UNSUPPORTED;
	}
    
    VOSLayerRenderer = OXRVisionOS::GetSwiftLayerRenderer();

	// Create World Tracking data provider
	ar_world_tracking_configuration_t ArKitWorldTrackingConfiguration = ar_world_tracking_configuration_create();
	ARKitWorldTrackingProvider = ar_world_tracking_provider_create(ArKitWorldTrackingConfiguration);
	
	// Create Hand Tracking data provider
	BeginHandTracking();
	ar_hand_tracking_configuration_t ArKitHandTrackingConfiguration = ar_hand_tracking_configuration_create();
	ARKitHandTrackingProvider = ar_hand_tracking_provider_create(ArKitHandTrackingConfiguration);
	
	// Create Data providers
	ar_data_providers_t ARKitDataProviders = ar_data_providers_create();
	// Add
	ar_data_providers_add_data_provider(ARKitDataProviders, ARKitWorldTrackingProvider);
	ar_data_providers_add_data_provider(ARKitDataProviders, ARKitHandTrackingProvider);

	// Start the arkit session
	ar_session_run(ARKitSession, ARKitDataProviders);

	// If we failed to get access to the hmd we could fail here like this.
	// if (???)
	// {
	// 	UE_LOGF(LogOXRVisionOS, Warning, "Open failed code 0x%x.  Session going to XR_SESSION_STATE_LOSS_PENDING.", HMDHandle);
	// 	SetSessionState(XrSessionState::XR_SESSION_STATE_LOSS_PENDING);
	// 	return XrResult::XR_SESSION_LOSS_PENDING;
	// }

	if (Tracker->RegisterDevice(TrackerDeviceType::DEVICE_VR_HMD, HMDHandle) == false)
	{
		UE_LOGF(LogOXRVisionOS, Error, "Tracker->RegisterDevice failed.  Bug.");
		return XrResult::XR_ERROR_RUNTIME_FAILURE;
	}

    // Fill the game threads view info buffer with temporary data, some of which was collected in FSwiftAppBootstrap::KickoffWithCompositingLayer
	// XrLocateViews will overwrite this with real data fetched from the VisionOS API, but we will use this temp fov data in the first two game thread frames.
    LocateViewInfoBuffer[0].NumViews = OXRVisionOS::GetSwiftNumViewports();
    LocateViewInfoBuffer[0].ViewTransforms[0] = matrix_identity_float4x4;
    LocateViewInfoBuffer[0].ViewTransforms[1] = matrix_identity_float4x4;
    simd_float4x4 ViewTransforms[2];
    const float HalfFov = 2.19686294f / 2.f;
    LocateViewInfoBuffer[0].HmdFovs[0].angleLeft = -HalfFov - 0.1f;
    LocateViewInfoBuffer[0].HmdFovs[0].angleRight = HalfFov - 0.1f;
    LocateViewInfoBuffer[0].HmdFovs[0].angleUp = HalfFov;
    LocateViewInfoBuffer[0].HmdFovs[0].angleDown = -HalfFov;
    LocateViewInfoBuffer[0].HmdFovs[1].angleLeft = -HalfFov + 0.1f;
    LocateViewInfoBuffer[0].HmdFovs[1].angleRight = HalfFov + 0.1f;
    LocateViewInfoBuffer[0].HmdFovs[1].angleUp = HalfFov;
    LocateViewInfoBuffer[0].HmdFovs[1].angleDown = HalfFov;
    LocateViewInfoBuffer[0].HmdFovs[1] = LocateViewInfoBuffer[0].HmdFovs[0];
    LocateViewInfoBuffer[1] = LocateViewInfoBuffer[0];
    LocateViewInfoBuffer[2] = LocateViewInfoBuffer[0];

	bIsRunning = true;
	PipelinedFrameStateGame.bSynchronizing = true;
	PipelinedFrameStateRendering.bSynchronizing = true;
	PipelinedFrameStateRHI.bSynchronizing = true;

	//Controllers->OnBeginSession(Tracker.Get());

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrEndSession()
{
	if (!IsSessionRunning())
	{
		return XrResult::XR_ERROR_SESSION_NOT_RUNNING;
	}

	if (SessionState != XrSessionState::XR_SESSION_STATE_STOPPING)
	{
		return XrResult::XR_ERROR_SESSION_NOT_STOPPING;
	}

	EndSessionInternal();

	SetSessionState(XrSessionState::XR_SESSION_STATE_IDLE);
	// Start polling to go back to XR_SESSION_STATE_READY as soon as the hmd is available
	OnWorldTickStartDelegateHandle = FWorldDelegates::OnWorldTickStart.AddRaw(this, &FOXRVisionOSSession::OnWorldTickStart);

	return XrResult::XR_SUCCESS;
}

void FOXRVisionOSSession::EndSessionInternal()
{
	UE_LOGF(LogOXRVisionOS, Warning, "FOXRVisionOSSession::EndSessionInternal()");

	//Controllers->OnEndSession(Tracker.Get());

	Tracker->UnregisterDevice(HMDHandle);
	//Close(HMDHandle);
	HMDHandle = INVALID_DEVICE_HANDLE;

	if (bExitRequested)
	{
		SetSessionState(XrSessionState::XR_SESSION_STATE_EXITING);
	}

	bExitRequested = false;
	bIsRunning = false;
}

XrResult FOXRVisionOSSession::XrRequestExitSession()
{
	if (!IsSessionRunning())
	{
		return XrResult::XR_ERROR_SESSION_NOT_RUNNING;
	}

	SetSessionState(XrSessionState::XR_SESSION_STATE_STOPPING);
	bExitRequested = true;
	return XrResult::XR_SUCCESS;
}

// FOpenXRHMD::OnBeginSimulation_GameThread()
XrResult FOXRVisionOSSession::XrWaitFrame(
	const XrFrameWaitInfo* frameWaitInfo,
	XrFrameState* frameState)
{
	//UE_LOGF(LogOXRVisionOS, Log, "FOXRVisionOSSession XrWaitFrame called");

	SCOPED_NAMED_EVENT_TEXT("FOXRVisionOSSession::XrWaitFrame", FColor::Turquoise);

	// frameWaitInfo can be null
	if (frameWaitInfo != nullptr && frameWaitInfo->type != XR_TYPE_FRAME_WAIT_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (frameState == nullptr || frameState->type != XR_TYPE_FRAME_STATE)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (!IsSessionRunning())
	{
		UE_LOGF(LogOXRVisionOS, Log, "FOXRVisionOSSession XrWaitFrame XR_ERROR_SESSION_NOT_RUNNING");
		return XrResult::XR_ERROR_SESSION_NOT_RUNNING;
	}
	
	// Block until the previous frame's xrBeginFrame has happened.
	{
		SCOPED_NAMED_EVENT_TEXT("FOXRVisionOSSession::XrWaitFrame XrWaitFrameEvent->Wait()", FColor::Turquoise);
		UE_LOGF(LogOXRVisionOS, Verbose, "XrWaitFrameEvent->Wait() Started   FC will be %i", SessionFrameCounter);
		XrWaitFrameEvent->Wait();
	}
	
	XrResult SuccessfulReturnValue =  XrResult::XR_SUCCESS;
	bool bSetSyncronizedThisFrame = false;
	{
		FScopeLock Lock(&CpLayerMutex);
		switch (cp_layer_renderer_get_state(OXRVisionOS::GetSwiftLayerRenderer()))
		{
			case cp_layer_renderer_state_paused:
				// Handle app pause, this happens when you take the device off your head.
				// We want to call cp_layer_renderer_wait_until_running to allow the system to take over, however
				// we need in flight frames to complete first so that we don't keep making api calls
				// and we want to call that on the rendering thread because apple only allows these functions to
				// be called from a single thread.
				// So we will go to the synchronized state for this frame and call cp_layer_renderer_wait_until_running
				// in XrEndFrame instead of doing a normal submission to the compositor.  That will block the render thread.
				// When that blocking call returns our next new frame will transition back to VISIBILE then FOCUSED below.
				UE_LOGF(LogOXRVisionOS, Log, "FOXRVisionOSSession XrWaitFrame cp_layer_renderer_get_state returned cp_layer_renderer_state_paused.  Transitioning to XR_SESSION_STATE_SYNCHRONIZED, shouldRender will be false.");
				if (GetSessionState() != XrSessionState::XR_SESSION_STATE_SYNCHRONIZED)
				{
					SetSessionState(XrSessionState::XR_SESSION_STATE_SYNCHRONIZED);
				}
				PipelinedFrameStateGame.bSynchronizing = true;
				bSetSyncronizedThisFrame = true;
				break;
				
			case cp_layer_renderer_state_running:
				break;
				
			case cp_layer_renderer_state_invalidated:
				// Not entirely sure this is correct, but it's kind of close.
				UE_LOGF(LogOXRVisionOS, Log, "FOXRVisionOSSession XrWaitFrame LayerRendererState == cp_layer_renderer_state_invalidated.  SetSessionState to XR_SESSION_STATE_LOSS_PENDING.");
				SetSessionState(XrSessionState::XR_SESSION_STATE_LOSS_PENDING);
				SuccessfulReturnValue = XrResult::XR_SESSION_LOSS_PENDING;
				PipelinedFrameStateGame.bSessionLost = true;
				break;
			default:
				check(false);
		}
	}
	
	if (GetSessionState() == XrSessionState::XR_SESSION_STATE_READY)
	{
		SetSessionState(XrSessionState::XR_SESSION_STATE_SYNCHRONIZED);
	}
	
	if (GetSessionState() == XrSessionState::XR_SESSION_STATE_SYNCHRONIZED && !bSetSyncronizedThisFrame)
	{
		SetSessionState(XrSessionState::XR_SESSION_STATE_VISIBLE);
		SetSessionState(XrSessionState::XR_SESSION_STATE_FOCUSED);

		SessionFrameCounter += OXRVisionOSSessionHelpers::OXRVisionOSBackbufferLength;  // Jump ahead in case the current frame counter was partially used while the previous session was being ended.
		PipelinedFrameStateGame.bSynchronizing = false;
	}

	PipelinedFrameStateGame.FrameCounter = SessionFrameCounter++;
	// Each frame we read this index on the game thread and then we write the index after this on the render thread.
	PipelinedFrameStateGame.RenderToGameFrameStateIndex++;
	UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i XrWaitFrameEvent->Wait() Finished", VOSThreadString(), PipelinedFrameStateGame.FrameCounter);
    	

	XrTime DisplayPeriod = 6 * 1000000; //TODO other frame sequences?  This is 90fps
	frameState->predictedDisplayPeriod = DisplayPeriod;
	
	RenderToGameFrameStateRead(PipelinedFrameStateGame);
	XrTime EarlyPredictionTime = OXRVisionOS::APITimeToXrTime(PipelinedFrameStateGame.PreviousPredictionTime);
	EarlyPredictionTime += DisplayPeriod;
	
	frameState->predictedDisplayTime = EarlyPredictionTime;
	
	UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x FOXRVisionOSSession::XrWaitFrame estimated XrTime predictedDisplayTime=%lld", VOSThreadString(), PipelinedFrameStateGame.FrameCounter, PipelinedFrameStateGame.SwiftFrame, frameState->predictedDisplayTime);

    LocateViewInfoIndexAdvance();

	frameState->shouldRender = SessionState == XR_SESSION_STATE_FOCUSED || SessionState == XR_SESSION_STATE_VISIBLE;
	PipelinedFrameStateGame.bShouldRender = frameState->shouldRender;

	// We have to pipeline state updates in order to prevent task overlap (esp during map loading where we kick extra renders)
	ENQUEUE_RENDER_COMMAND(UpdatePipelinedFrameState)([this, GameFrameState = PipelinedFrameStateGame](FRHICommandListImmediate& RHICmdList)
	{
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i  UpdatePipelinedFrameState G->R ", VOSThreadString(), GameFrameState.FrameCounter);
		check(IsInRenderingThread());
		
		FPipelinedFrameState& FrameState = PipelinedFrameStateRendering;
		FrameState = GameFrameState;
		
		if (FrameState.bSynchronizing) 
		{
			UE_LOGF(LogOXRVisionOS, Verbose, "%ls FC_%i  FOXRVisionOSSession::XrWaitFrame FrameState.bSynchronizing, not getting a frame.", VOSThreadString(), FrameState.FrameCounter);
			return;
		}
		
		if (PipelinedFrameStateGame.bSessionLost)
		{
			UE_LOGF(LogOXRVisionOS, Verbose, "%ls FC_%i  FOXRVisionOSSession::XrWaitFrame FrameState.bSessionLost, not getting a frame.", VOSThreadString(), FrameState.FrameCounter);
			return;
		}
		
		{
			FScopeLock Lock(&CpLayerMutex);
			FrameState.SwiftFrame = cp_layer_renderer_query_next_frame(VOSLayerRenderer);
		}
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x  FOXRVisionOSSession::XrWaitFrame cp_layer_renderer_query_next_frame SwiftLayerFrame fetched", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame);

		if (FrameState.SwiftFrame) 
		{
			// Fetch the predicted timing information.
			FrameState.SwiftFrameTiming = cp_frame_predict_timing(FrameState.SwiftFrame);
			if ( FrameState.SwiftFrameTiming == nullptr)
			{
			 //TODO ??? failed frame?
			 // This means the layer is not in the correct state  Perhaps we are either running too early or we need to end the session.
			 // we might want to return XR_ERROR_SESSION_NOT_RUNNING here, but only if we actually end the session first.
			 assert(false);
			 //return;
			}

			CFTimeInterval TargetRenderTimeInterval = cp_time_to_cf_time_interval(cp_frame_timing_get_presentation_time(FrameState.SwiftFrameTiming));

			UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x FOXRVisionOSSession::XrWaitFrame cp_frame_predict_timing gave xr TargetRenderTimeInterval=%f",
				VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame,
				(double)TargetRenderTimeInterval);

			// Mark the beginnng of our game thread update work.  All the stuff before rendering.
			UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x FOXRVisionOSSession::XrWaitFrame cp_frame_start_update", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame);
			cp_frame_start_update(FrameState.SwiftFrame);

			FrameState.PredictedDisplayTime = OXRVisionOS::APITimeToXrTime(TargetRenderTimeInterval);

			cp_frame_end_update(FrameState.SwiftFrame);
		}
		else
		{
			UE_LOGF(LogOXRVisionOS, Log, "%ls FC_%i SLF_0x%x  FOXRVisionOSSession::XrWaitFrame cp_layer_renderer_query_next_frame got null frame, if the session is stopping this is ok.", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame);
		}
	});

	// Update the STAGE space
	//Tracker->UpdateReferenceSpaces(Instance, this, HMDHandle, PipelinedFrameStateGame.PredictedDisplayTime, PipelinedFrameStateGame.FrameCounter);

	return SuccessfulReturnValue;
}

void FOXRVisionOSSession::OnBeginRendering_GameThread()
{
    // Mark the end of our game frame
	// Currently we are going through this in UpdatePipelinedFrameState, because all cp_ calls have to happen on the render thread.  Perhaps in the future we will be able to track the game thread work.
//    if (PipelinedFrameStateGame.SwiftFrame)
//    {
//		UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x cp_frame_end_update in FOXRVisionOSSession::OnBeginRendering_GameThread", VOSThreadString(), PipelinedFrameStateGame.FrameCounter, PipelinedFrameStateGame.SwiftFrame);
//        cp_frame_end_update(PipelinedFrameStateGame.SwiftFrame);
//        PipelinedFrameStateGame.SwiftFrame = nullptr;
//    }
}

// FOpenXRHMD::OnBeginRendering_RenderThread
void FOXRVisionOSSession::OnBeginRendering_RenderThread_PreDeviceLocationUpdate()
{
	UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x FOXRVisionOSSession::OnBeginRendering_RenderThread_PreDeviceLocationUpdate", VOSThreadString(), PipelinedFrameStateRendering.FrameCounter, PipelinedFrameStateRendering.SwiftFrame);
	
	WaitUntil();
    StartSubmission();
	
	UE_LOGF(LogOXRVisionOS, VeryVerbose, "Enqueueing framestate from render to rhi: NewFrameState.FrameCounter %i", PipelinedFrameStateRendering.FrameCounter);
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.EnqueueLambda([this, NewRHIFrameState = PipelinedFrameStateRendering](FRHICommandListImmediate&)
	{
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x UpdatePipelinedFrameStat R->I, but HeadTransform Cleared", VOSThreadString(), NewRHIFrameState.FrameCounter, NewRHIFrameState.SwiftFrame);
		
		PipelinedFrameStateRHI = NewRHIFrameState;
		PipelinedFrameStateRHI.HeadTransform = matrix_identity_float4x4; // overwrite to ensure rhi gets a new one before it uses one.
	});
}

// FOpenXRHMD::OnBeginRendering_RHIThread
XrResult FOXRVisionOSSession::XrBeginFrame(
	const XrFrameBeginInfo* FrameBeginInfo)
{
	check(!IsRHIThreadRunning() || IsInRHIThread());

	SCOPED_NAMED_EVENT_TEXT("FOXRVisionOSSession::XrBeginFrame", FColor::Turquoise);
	
	UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x XrBeginFrame", VOSThreadString(), PipelinedFrameStateRHI.FrameCounter, PipelinedFrameStateRHI.SwiftFrame, PipelinedFrameStateRHI.SwiftDrawable);

	// frameBeginInfo can be null
	if (FrameBeginInfo != nullptr && FrameBeginInfo->type != XR_TYPE_FRAME_BEGIN_INFO)
	{
		XrWaitFrameEvent->Trigger();
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (!IsSessionRunning())
	{
		XrWaitFrameEvent->Trigger();
		return XrResult::XR_ERROR_SESSION_NOT_RUNNING;
	}

	if (PipelinedFrameStateRHI.FrameCounter == CachedBeginFlipFrameCounter)
	{
		// We don't trigger in this case because there wasn't a call to xrWaitFrame that corresponds
		// xrWaitFrame should have incremented the counter
		UE_LOGF(LogOXRVisionOS, Log, "XrWaitFrameEvent not triggered because call order invalid  RHIFrameCounter %i != RenderFrameCounter %i", PipelinedFrameStateRHI.FrameCounter, CachedBeginFlipFrameCounter);
		return XrResult::XR_ERROR_CALL_ORDER_INVALID;
	}
    CachedBeginFlipFrameCounter = PipelinedFrameStateRHI.FrameCounter;
	
	if (PipelinedFrameStateRHI.bSynchronizing || PipelinedFrameStateRHI.bSessionLost)
	{
		XrWaitFrameEvent->Trigger();
		return XrResult::XR_SUCCESS;
	}

	XrWaitFrameEvent->Trigger();

	const XrResult BeginFrameResult = (NextBackBufferIndex != PreviousBackBufferIndexInBegin) ? XrResult::XR_SUCCESS : XrResult::XR_FRAME_DISCARDED;

	//if (PipelinedFrameStateRHI.CommandListContext)
    {
        // TODO
        // I think there's gonna be some trickiness when we have accurate VISIBLE + FOCUSED session states.
        // If we return XrFrameState.shouldRender == false (headset off), we still could deploy WaitUntilSafeForRendering
        // either here or in xrWaitSwapchainImage. In that case, we will hang because the flip won't deploy.
        // We might want to reconsider using WaitUntilSafeForRendering. If we use xrWaitFrame or xrWaitSwapchainImage
        // to wait for a CPU-signal for swapchain safety, then...we don't really need to check on the GPU again.
        // But the docs infer that ReprojectionBeginFrame and waitUntilSafeForRendering are tied :/
    }
    
    //MetalRHIOXRVisionOS::WaitUntilSafeForRendering(PipelinedFrameStateRHI.CommandListContext, NextBackBufferIndex);
    PreviousBackBufferIndexInBegin = NextBackBufferIndex;

	return BeginFrameResult;
}

void FOXRVisionOSSession::WaitUntil()
{
    FPipelinedFrameState& FrameState = GetPipelinedFrameStateForThread();
    
	if (FrameState.SwiftFrame)
	{
		cp_time_t OptimalInputTime = cp_frame_timing_get_optimal_input_time(FrameState.SwiftFrameTiming);
		CFTimeInterval TimeInterval = cp_time_to_cf_time_interval(OptimalInputTime);
		
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::WaitUntil cp_time_wait_until started waiting until OptimalInputTime %f", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable, TimeInterval);
		
		cp_time_wait_until(OptimalInputTime);
		
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::WaitUntil cp_time_wait_until completed", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable);
	}
	else
	{
		UE_LOGF(LogOXRVisionOS, Verbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::WaitUntil doing nothing because SwiftFrame is null", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable);
	}
}

void FOXRVisionOSSession::StartSubmission()
{
    FPipelinedFrameState& FrameState = GetPipelinedFrameStateForThread();
	
	if (FrameState.SwiftFrame)
	{
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::StartSubmission cp_frame_start_submission", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable);
		
		FrameState.SwiftDrawable = cp_frame_query_drawable(FrameState.SwiftFrame);	
		
		if (FrameState.SwiftDrawable) 
		{
			cp_frame_start_submission(FrameState.SwiftFrame);				
			
			UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::StartSubmission cp_frame_query_drawable in got the SD again.", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable);
			
			FrameState.SwiftFinalFrameTiming = cp_drawable_get_frame_timing(FrameState.SwiftDrawable);
			FrameState.SwiftFinalFrameTimeInterval = cp_time_to_cf_time_interval(cp_frame_timing_get_presentation_time(FrameState.SwiftFinalFrameTiming));
			FrameState.DeviceAnchor = ar_device_anchor_create();
			
			UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::StartSubmission cp_drawable_get_frame_timing got SwiftFinalFrameTimeInterval %f", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable, (double)FrameState.SwiftFinalFrameTimeInterval);
			
			uint64_t CurrentMachTime = mach_absolute_time();
			uint64_t PredictedPresentationMachTime = cp_frame_timing_get_presentation_time(FrameState.SwiftFinalFrameTiming).cp_mach_abs_time;
			UE_LOGF(LogOXRVisionOS, VeryVerbose, "   FC_%i                   FOXRVisionOSSession::StartSubmission CurrentMachTime %lld PredictedPresentationMachTime %lld", FrameState.FrameCounter,
				   CurrentMachTime, PredictedPresentationMachTime);
			
			check(ARKitWorldTrackingProvider);
			
			{
				auto anchor_status = ar_world_tracking_provider_query_device_anchor_at_timestamp(ARKitWorldTrackingProvider, FrameState.SwiftFinalFrameTimeInterval, FrameState.DeviceAnchor);
				if (anchor_status == ar_device_anchor_query_status_success)
				{
					cp_drawable_set_device_anchor(FrameState.SwiftDrawable, FrameState.DeviceAnchor);
					
					// Get the HMD transform and cache it
					FrameState.HeadTransform = ar_anchor_get_origin_from_anchor_transform(FrameState.DeviceAnchor);
				}
				else
				{
					UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x  FOXRVisionOSSession::StartSubmission cp_drawable_set_device_anchor DeviceAnchor query failed, setting nullptr, no reprojection", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable);
					
					cp_drawable_set_device_anchor(FrameState.SwiftDrawable, nullptr);
					// Leave FrameState.HeadTransform untouched, same as game thread. It may simply be identity.  But whatever it is we will go ahead and render with it.
				}
			}
		}
		else 
		{
			UE_LOGF(LogOXRVisionOS, Log, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::StartSubmission doing nothing because SwiftDrawable is null.  This is OK if the app is being paused or exiting.", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable);			
		}
	}
	else
	{
		UE_LOGF(LogOXRVisionOS, Verbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::StartSubmission doing nothing because SwiftFrame is null", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable);
	}
	RenderToGameFrameStateWrite(FrameState);
}

// FOpenXRHMD::OnFinishRendering_RHIThread()
XrResult FOXRVisionOSSession::XrEndFrame(
	const XrFrameEndInfo* InFrameEndInfo)
{
	UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::XrEndFrame", VOSThreadString(), PipelinedFrameStateRHI.FrameCounter, PipelinedFrameStateRHI.SwiftFrame, PipelinedFrameStateRHI.SwiftDrawable);

	SCOPED_NAMED_EVENT_TEXT("FOXRVisionOSSession::XrEndFrame", FColor::Turquoise);

	if (InFrameEndInfo == nullptr || InFrameEndInfo->type != XR_TYPE_FRAME_END_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (!IsSessionRunning())
	{
		return XrResult::XR_ERROR_SESSION_NOT_RUNNING;
	}
	
	const XrRHIContextEPIC* RHIContextEPIC = OpenXR::FindChainedStructByType<XrRHIContextEPIC>(InFrameEndInfo, (XrStructureType)XR_TYPE_RHI_CONTEXT_EPIC);
	check(RHIContextEPIC && RHIContextEPIC->RHIContext);

	if (PipelinedFrameStateRHI.bSynchronizing)
	{
		UE_LOGF(LogOXRVisionOS, Log, "FOXRVisionOSSession XrEndFrame bSynchronizing=true.  Calling cp_layer_renderer_wait_until_running");
		
		{
			FScopeLock Lock(&CpLayerMutex);
			cp_layer_renderer_wait_until_running(VOSLayerRenderer);
		}
		UE_LOGF(LogOXRVisionOS, Log, "FOXRVisionOSSession XrEndFrame cp_layer_renderer_wait_until_running returned");
		
		// We need to call present so that it can signal the FrameSemaphore.
		const MetalRHIVisionOS::PresentImmersiveParams Params{MetalRHIVisionOS::EPresentChoice::DoNotPresent, FTextureRHIRef(), FTextureRHIRef(), PipelinedFrameStateRHI.SwiftFrame, PipelinedFrameStateRHI.SwiftDrawable, PipelinedFrameStateRHI.FrameCounter, RHIContextEPIC->RHIContext};
		MetalRHIVisionOS::PresentImmersive(Params);
		return XrResult::XR_SUCCESS;
	}
	
	if (PipelinedFrameStateRHI.bSessionLost)
	{
		// We need to call present so that it can signal the FrameSemaphore.
		const MetalRHIVisionOS::PresentImmersiveParams Params{MetalRHIVisionOS::EPresentChoice::DoNotPresent, FTextureRHIRef(), FTextureRHIRef(), PipelinedFrameStateRHI.SwiftFrame, PipelinedFrameStateRHI.SwiftDrawable, PipelinedFrameStateRHI.FrameCounter, RHIContextEPIC->RHIContext};
		MetalRHIVisionOS::PresentImmersive(Params);
		return XrResult::XR_SUCCESS;
	}
	
	if (PipelinedFrameStateRHI.SwiftFrame == nullptr)
	{
		UE_LOGF(LogOXRVisionOS, Verbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::XrEndFrame with null SwiftFrame, not submitting", VOSThreadString(), PipelinedFrameStateRHI.FrameCounter, PipelinedFrameStateRHI.SwiftFrame, PipelinedFrameStateRHI.SwiftDrawable);
		return XrResult::XR_SUCCESS;
	}

	const XrFrameEndInfo& FrameEndInfo = *InFrameEndInfo;
	//check(FrameEndInfo.environmentBlendMode == XR_ENVIRONMENT_BLEND_MODE_OPAQUE); // only opaque is supported, see XrEnumerateEnvironmentBlendModes
	check(FrameEndInfo.layerCount <= OXRVisionOSSessionHelpers::MaxOXRVisionOSLayers);
    
    {
        //TODO??? supporting only one layer now.  This means we do not have a non-reprojected layer.  Maybe we could have one as a separate swiftui?
        check(FrameEndInfo.layerCount == 1);
        check(FrameEndInfo.layers != nullptr);
		if (FrameEndInfo.layers[0] == nullptr) {
			return XrResult::XR_ERROR_LAYER_INVALID;
		}
		
        const XrCompositionLayerBaseHeader& LayerBaseHeader = *FrameEndInfo.layers[0];
        check(LayerBaseHeader.type == XR_TYPE_COMPOSITION_LAYER_PROJECTION);
        const XrCompositionLayerProjection& LayerProjection = *(reinterpret_cast<const XrCompositionLayerProjection*>(FrameEndInfo.layers[0]));
        check(LayerProjection.viewCount == 2); // Must be the number of views returned by xrLocateViews
        const XrCompositionLayerProjectionView& View0 = LayerProjection.views[0];
        const XrCompositionLayerProjectionView& View1 = LayerProjection.views[1];

        check(View0.subImage.swapchain == View1.subImage.swapchain); //TODO: should we do it this way or should we have two separate swapchains?  If we did we would need to handle both below.
        
        const FOXRVisionOSSwapchain& SwapchainData0 = *(reinterpret_cast<FOXRVisionOSSwapchain*>(View0.subImage.swapchain));
        const FOXRVisionOSSwapchain::FSwapchainImage& WaitedImage = SwapchainData0.GetLastWaitedImage();
		
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::XrEndFrame SwapchainData0.GetLastWaitedImage() metal textgure = 0x%x", VOSThreadString(), PipelinedFrameStateRHI.FrameCounter, PipelinedFrameStateRHI.SwiftFrame, PipelinedFrameStateRHI.SwiftDrawable, WaitedImage.Image.GetReference()->GetNativeResource());
		
		const XrCompositionLayerDepthInfoKHR* Depth0 = OpenXR::FindChainedStructByType<XrCompositionLayerDepthInfoKHR>(View0.next, (XrStructureType)XR_TYPE_COMPOSITION_LAYER_DEPTH_INFO_KHR);
		const XrCompositionLayerDepthInfoKHR* Depth1 = OpenXR::FindChainedStructByType<XrCompositionLayerDepthInfoKHR>(View1.next, (XrStructureType)XR_TYPE_COMPOSITION_LAYER_DEPTH_INFO_KHR);
		
		check(Depth0 && Depth1);
		check(Depth0->subImage.swapchain == Depth1->subImage.swapchain); //TODO: should we do it this way or should we have two separate swapchains?  If we did we would need to handle both below.

		const FOXRVisionOSSwapchain& SwapchainDepthData0 = *(reinterpret_cast<FOXRVisionOSSwapchain*>(Depth0->subImage.swapchain));
		const FOXRVisionOSSwapchain::FSwapchainImage& WaitedDepth = SwapchainDepthData0.GetLastWaitedImage();

		const MetalRHIVisionOS::PresentImmersiveParams Params{MetalRHIVisionOS::EPresentChoice::Present, WaitedImage.Image, WaitedDepth.Image, PipelinedFrameStateRHI.SwiftFrame, PipelinedFrameStateRHI.SwiftDrawable, PipelinedFrameStateRHI.FrameCounter, RHIContextEPIC->RHIContext};
        MetalRHIVisionOS::PresentImmersive(Params);
    }

	NextBackBufferIndex = (NextBackBufferIndex + 1) % OXRVisionOSSessionHelpers::OXRVisionOSBackbufferLength;

	return XrResult::XR_SUCCESS;
}

XrResult FOXRVisionOSSession::XrLocateViews(
	const XrViewLocateInfo*		ViewLocateInfo,
	XrViewState*				ViewState,
	uint32_t                    ViewCapacityInput,
	uint32_t*					ViewCountOutput,
	XrView*						Views)
{
    if (ViewLocateInfo == nullptr || ViewLocateInfo->type != XR_TYPE_VIEW_LOCATE_INFO)
    {
        return XrResult::XR_ERROR_VALIDATION_FAILURE;
    }
    
    if (ViewState == nullptr)
    {
        return XrResult::XR_ERROR_VALIDATION_FAILURE;
    }
    
    if (ViewCapacityInput != 0 && Views == nullptr)
    {
        return XrResult::XR_ERROR_VALIDATION_FAILURE;
    }
    
    if (ViewLocateInfo->viewConfigurationType != XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO)
    {
        return XrResult::XR_ERROR_VIEW_CONFIGURATION_TYPE_UNSUPPORTED;
    }
    
	const FOXRVisionOSSpace* BaseSpacePtr = ToFOXRVisionOSSpace(ViewLocateInfo->space);
	if (BaseSpacePtr == nullptr) {
		return XrResult::XR_ERROR_HANDLE_INVALID;
	}
    
	FPipelinedFrameState& FrameState = GetPipelinedFrameStateForThread();
	
	//UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::XrLocateViews", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable);

	bool bUsePreviousFrameViews = false;
	FLocateViewInfo PreviousViewInfo;
	if (IsInGameThread())
	{
		// In the game thread we use the previous render thread views
		PreviousViewInfo = GetLocateViewInfo_GameThread();
		bUsePreviousFrameViews = true;
	}
	else if (FrameState.SwiftDrawable == nullptr)
	{
		// If a frame is abandoned we use the previous render thread views
		PreviousViewInfo = GetLocateViewInfo_RenderThread();
		bUsePreviousFrameViews = true;
	}
	
	uint32_t NumViews = 0;
    if (bUsePreviousFrameViews)
    {
        NumViews = PreviousViewInfo.NumViews;
    }
    else
    {
        check(IsInRenderingThread());
        check(FrameState.SwiftDrawable);
        NumViews = cp_drawable_get_view_count(FrameState.SwiftDrawable);
        FLocateViewInfo& ViewInfoCache = GetLocateViewInfo_RenderThread();
        ViewInfoCache.NumViews = NumViews;
    }

    //TODO probably all of this should happen only once per frame in xrBeginFrame and once in xrWaitFrame.
    
	*ViewCountOutput = NumViews;
	if (ViewCapacityInput != 0)
    {
        if (ViewCapacityInput < *ViewCountOutput)
        {
            return XrResult::XR_ERROR_SIZE_INSUFFICIENT;
        }
        
        // The view is always located.
        ViewState->viewStateFlags = (XR_VIEW_STATE_POSITION_VALID_BIT |
									 XR_VIEW_STATE_POSITION_TRACKED_BIT |
									 XR_VIEW_STATE_ORIENTATION_VALID_BIT |
									 XR_VIEW_STATE_ORIENTATION_TRACKED_BIT);
        
        for (int Index = 0; Index < NumViews; ++Index)
        {
            if (bUsePreviousFrameViews)
            {
                Views[Index].pose = OXRVisionOS::ToXrPose(PreviousViewInfo.ViewTransforms[Index]);
                Views[Index].fov = PreviousViewInfo.HmdFovs[Index];
            }
            else
            {
                check(IsInRenderingThread());
                check(FrameState.SwiftDrawable);

				cp_view_t View = cp_drawable_get_view(FrameState.SwiftDrawable, Index);
				
#if VISIONOS_MAJOR_VERSION == 1
                simd_float4 Tangents = cp_view_get_tangents(View);
				//simd_float2 DepthRange = cp_drawable_get_depth_range(FrameState.SwiftDrawable);  //TODO ??? do we need this?  There is an openxrextension for it, could use that.
    //                                                                          DepthRange[1], /* nearZ */
    //                                                                          DepthRange[0], /* farZ */
    //                                                                          true); /* reverseZ */
                
                FrameState.HmdFovs[Index].angleLeft     = -FMath::Atan(Tangents[0]);
				FrameState.HmdFovs[Index].angleRight    =  FMath::Atan(Tangents[1]);
                FrameState.HmdFovs[Index].angleUp       =  FMath::Atan(Tangents[2]);
                FrameState.HmdFovs[Index].angleDown     = -FMath::Atan(Tangents[3]);
#elif VISIONOS_MAJOR_VERSION >= 2
				simd_float4x4 ProjectionMatrix = cp_drawable_compute_projection(FrameState.SwiftDrawable, cp_axis_direction_convention_right_up_back, Index);
				
				// Extract the tangents from the projection matrix
				const float InvLmR = -ProjectionMatrix.columns[0][0] / 2.0f;
				const float InvBmT = -ProjectionMatrix.columns[1][1] / 2.0f; 
				const float SumRL = ProjectionMatrix.columns[2][0] / -InvLmR;
				const float SumTB = ProjectionMatrix.columns[2][1] / -InvBmT;
				const float LMinusR = 1.0 / InvLmR;
				const float BMinusT = 1.0 / InvBmT;
				const float L = (SumRL + LMinusR) / 2.0f;
				const float R = SumRL - L;
				const float B = (SumTB + BMinusT) / 2.0f;
				const float T = SumTB - B;
				
				FrameState.HmdFovs[Index].angleLeft     = FMath::Atan(L);
				FrameState.HmdFovs[Index].angleRight    = FMath::Atan(R);
				FrameState.HmdFovs[Index].angleUp       = FMath::Atan(T);
				FrameState.HmdFovs[Index].angleDown     = FMath::Atan(B);
#else
	static_assert(false); // Something wrong with the VISIONOS_MAJOR_VERSION define
#endif
                // Hack: We only xrLocateViews for the HMD device, and cp_view_get_transform gives us hmd relative transforms, so we don't need to do any additional math here.
				// However OpenXR spec allows one to request the views in any space, which would require us to do some transforms and inverse transforms.
				// Perhaps OpenXRHMD ought to be getting the view poses in view space rather than HMD device space because we could detect that here easily???
                simd_float4x4 ViewTransform = cp_view_get_transform(View);
                Views[Index].pose = OXRVisionOS::ToXrPose(ViewTransform);
                Views[Index].fov = FrameState.HmdFovs[Index];

                // Cache the render thread data for the game thread
                FLocateViewInfo& ViewInfoCache = GetLocateViewInfo_RenderThread();
                ViewInfoCache.ViewTransforms[Index] = ViewTransform;
                ViewInfoCache.HmdFovs[Index] = FrameState.HmdFovs[Index];
            }
        }
	}

	return XrResult::XR_SUCCESS;
}

void FOXRVisionOSSession::GetHMDTransform(XrTime DisplayTime, FTransform& OutTransform, XrSpaceLocationFlags& OutLocationFlags)
{
    //TODO just ignoring XrTime right now!!!
    
    FPipelinedFrameState& FrameState = GetPipelinedFrameStateForThread();
	
    OutLocationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT
        | XR_SPACE_LOCATION_POSITION_VALID_BIT
        | XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT
    | XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
    
    OutTransform = OXRVisionOS::ToFTransform(FrameState.HeadTransform);

//	UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x FOXRVisionOSSession::GetHMDTransform XrHeadTransform x,y=%f,%f r=%f OutTransform x,y=%f,%f",
//		   VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable,
//		   FrameState.HeadTransform.columns[3][0], FrameState.HeadTransform.columns[3][1], FrameState.HeadTransform.columns[0][0],
//		   OutTransform.GetLocation().X, OutTransform.GetLocation().Y);
}

void FOXRVisionOSSession::GetTransformForButton(EOXRVisionOSControllerButton Button, XrTime DisplayTime, FTransform& OutTransform, XrSpaceLocationFlags& OutLocationFlags, FVector& OutLinearVelocity, FVector& OutAngularVelocity, XrSpaceVelocityFlags& OutVelocityFlags)
{
	OutLocationFlags = 0;
	OutVelocityFlags = 0;

//	bool bGaze = false;
	int32 DeviceHandle = INVALID_DEVICE_HANDLE;
	switch (Button)
	{
	case EOXRVisionOSControllerButton::GripL:
	case EOXRVisionOSControllerButton::AimL:
	case EOXRVisionOSControllerButton::GripSurfaceL:
		DeviceHandle = Controllers->GetLeftControllerHandle();
		break;
	case EOXRVisionOSControllerButton::GripR:
	case EOXRVisionOSControllerButton::AimR:
	case EOXRVisionOSControllerButton::GripSurfaceR:
		DeviceHandle = Controllers->GetRightControllerHandle();
		break;
	case EOXRVisionOSControllerButton::HMDPose:
        check(false); // should be using GetHMDTransform
		DeviceHandle = HMDHandle;
		break;
	// case EOXRVisionOSControllerButton::GazePose:
	// 	DeviceHandle = HMDHandle;
	// 	bGaze = true;
	// 	break;
	default:
		break;
	}
	if (DeviceHandle == INVALID_DEVICE_HANDLE)
	{
		OutTransform = FTransform::Identity;
		return;
	}

	// if (bGaze)
	// {
	// 	// We get the gaze origin from the hmd.
	// 	bool bSuccess = Tracker->GetGazeResult(DeviceHandle, DisplayTime, FrameNumber, GazeResultData, HMDResultData);
	// 	if (!bSuccess)
	// 	{
	// 		OutTransform = FTransform::Identity;
	// 		return;
	// 	}
	// 	OutLocationFlags = OXRVisionOS::ToXrSpaceLocationFlags(HMDResultData);

	// 	// gazeDirection is in gazeOriginPose space.
	// 	PoseData& HMDGazeOriginPose = HMDResultData.Hmd.GazeOriginPose;
	// 	FQuat GazeOriginQuat =
	// 	FVector GazeOriginVector =
	// 	FQuat GazeQuat =

	// 	OutTransform = FTransform(GazeQuat) * FTransform(GazeOriginQuat, GazeOriginVector);
	// }
	// else
	{
		OXRVisionOS::TrackerResultData ResultData = {};
		bool bSuccess = Tracker->GetResult(DeviceHandle, DisplayTime, ResultData);
		if (!bSuccess)
		{
			OutTransform = FTransform::Identity;
			return;
		}

		//OutLocationFlags = OXRVisionOS::ToXrSpaceLocationFlags(ResultData);

		if ((OutLocationFlags & (XR_SPACE_LOCATION_POSITION_VALID_BIT | XR_SPACE_LOCATION_ORIENTATION_VALID_BIT)))
		{
			//PoseData& Pose = DeviceHandle == HMDHandle ? hmdInfo.Pose : vrControllerInfo.Pose;
			OXRVisionOS::TrackerPoseData Pose; //TEMP

			// If part of the transform is invalid send along identity/zero.
			FQuat Quat = (OutLocationFlags & XR_SPACE_LOCATION_ORIENTATION_VALID_BIT) ? OXRVisionOS::ToFQuat(Pose.orientation) : FQuat::Identity;
			FVector Vector = (OutLocationFlags & XR_SPACE_LOCATION_POSITION_VALID_BIT) ? OXRVisionOS::ToFVector(Pose.position) : FVector(0.0f, 0.0f, 0.0f);

			OutTransform = FTransform(Quat, Vector);

			// api provides just one controller pose, which is a reasonable aim pose, we shall make both adjustible.
			if (Button == EOXRVisionOSControllerButton::GripL || Button == EOXRVisionOSControllerButton::GripR)
			{
				OutTransform.SetRotation(OutTransform.TransformRotation(GripPoseAdjustment));
			}
			else if (Button == EOXRVisionOSControllerButton::AimL || Button == EOXRVisionOSControllerButton::AimR)
			{
				OutTransform.SetRotation(OutTransform.TransformRotation(AimPoseAdjustment));
			}
			else if (Button == EOXRVisionOSControllerButton::GripSurfaceL || Button == EOXRVisionOSControllerButton::GripSurfaceR)
			{
				OutTransform.SetRotation(OutTransform.TransformRotation(GripSurfacePoseAdjustment));
			}
		}
	}
}

// FOpenXRInput::Tick, game thread, after xrSyncActions
void FOXRVisionOSSession::OnPostSyncActions()
{
}

XrTime FOXRVisionOSSession::GetCurrentTime() const
{	
    const CFAbsoluteTime CurrentTime = CFAbsoluteTimeGetCurrent();
	return OXRVisionOS::APITimeToXrTime(CurrentTime);
}

void FOXRVisionOSSession::SetSessionState(XrSessionState NewState)
{
	UE_LOGF(LogOXRVisionOS, Log, "FOXRVisionOSSession %ls -> %ls", OpenXRSessionStateToString(SessionState), OpenXRSessionStateToString(NewState));

	SessionState = NewState;

	XrEventDataSessionStateChanged Event;
	Event.type = XR_TYPE_EVENT_DATA_SESSION_STATE_CHANGED;
	Event.next = nullptr;
	Event.session = (XrSession)this;
	Event.state = SessionState;
	Event.time = GetCurrentTime();

	Instance->EnqueueEvent(Event);
}

bool FOXRVisionOSSession::IsSessionRunning() const
{
	return bIsRunning;
}

const FOXRVisionOSSession::FPipelinedFrameState& FOXRVisionOSSession::GetPipelinedFrameStateForThread() const
{
	if (IsInRHIThread())
	{
		return PipelinedFrameStateRHI;
	}
	else if (IsInRenderingThread())
	{
		return PipelinedFrameStateRendering;
	}
	else
	{
		check(IsInGameThread());
		return PipelinedFrameStateGame;
	}
}

FOXRVisionOSSession::FPipelinedFrameState& FOXRVisionOSSession::GetPipelinedFrameStateForThread()
{
	return const_cast<FOXRVisionOSSession::FPipelinedFrameState&>(static_cast<const FOXRVisionOSSession&>(*this).GetPipelinedFrameStateForThread());
}

// Read from the current index.
void FOXRVisionOSSession::RenderToGameFrameStateRead(FPipelinedFrameState& InOutFrameState)
{
	FOXRVisionOSSession::FRenderToGameFrameState& RTGFS = RenderToGameFrameState[InOutFrameState.RenderToGameFrameStateIndex % RenderToGameFrameStateBufferLength];
	InOutFrameState.HeadTransform = RTGFS.HeadTransform;
	InOutFrameState.PreviousPredictionTime = RTGFS.PredictionTime;
	
//	UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x  FOXRVisionOSSession::RenderToGameFrameStateRead HeadTransform01 x,y=%f,%f r=%f  PredictionTime=%f",
//		   VOSThreadString(), InOutFrameState.FrameCounter, InOutFrameState.SwiftFrame, InOutFrameState.SwiftDrawable,
//		   RTGFS.HeadTransform.columns[3][0], RTGFS.HeadTransform.columns[3][1], RTGFS.HeadTransform.columns[0][0], (double)RTGFS.PredictionTime);
}

// Write into the next index that the game thread will read.
void FOXRVisionOSSession::RenderToGameFrameStateWrite(const FPipelinedFrameState& FrameState)
{
	FOXRVisionOSSession::FRenderToGameFrameState& RTGFS = RenderToGameFrameState[(FrameState.RenderToGameFrameStateIndex + 1) % RenderToGameFrameStateBufferLength];
	RTGFS.HeadTransform = FrameState.HeadTransform;
	RTGFS.PredictionTime = FrameState.SwiftFinalFrameTimeInterval;
	
	//UE_LOGF(LogOXRVisionOS, VeryVerbose, "%ls FC_%i SLF_0x%x SD_0x%8x  FOXRVisionOSSession::RenderToGameFrameStateWrite HeadTransform01 x,y=%f,%f  PredictionTime=%f", VOSThreadString(), FrameState.FrameCounter, FrameState.SwiftFrame, FrameState.SwiftDrawable, RTGFS.HeadTransform.columns[3][0], RTGFS.HeadTransform.columns[3][1], (double)RTGFS.PredictionTime);
}


XrResult FOXRVisionOSSession::XrCreateHandTrackerEXT(
	const XrHandTrackerCreateInfoEXT*           createInfo,
	XrHandTrackerEXT*                           handTracker)
{
	check(createInfo->handJointSet == XR_HAND_JOINT_SET_DEFAULT_EXT);  // We only support the default.
	FOXRVisionOSHandTracker& NewHandTracker = GetHandTracker(createInfo->hand);
	NewHandTracker.bCreated = true;
	*handTracker = (XrHandTrackerEXT)&NewHandTracker;
	return XR_SUCCESS;
}

XrResult FOXRVisionOSSession::FOXRVisionOSHandTracker::XrDestroyHandTrackerEXT()
{
	bCreated = false;
	return XR_SUCCESS;
}

XrResult FOXRVisionOSSession::FOXRVisionOSHandTracker::XrLocateHandJointsEXT(
	const XrHandJointsLocateInfoEXT*            locateInfo,
	XrHandJointLocationsEXT*                    locations)
{
	//locateInfo->time // TODO: just ignoring time for now, we will always get the latest.
	//locateInfo->baseSpace  //TODO: ignoring this, always get in the tracking space.
	
	locations->isActive = bIsActive;
	locations->jointCount = XR_HAND_JOINT_COUNT_EXT;
	for (int i = 0; i < locations->jointCount; ++i) 
	{
		locations->jointLocations[i] = JointLocations[i];
	}
	
	XrPosef pose = locations->jointLocations[XR_HAND_JOINT_THUMB_TIP_EXT].pose;

	return XR_SUCCESS;
}

void FOXRVisionOSSession::BeginHandTracking()
{
	if (JointMap.IsEmpty()) 
	{
		JointMap.Reserve(XR_HAND_JOINT_COUNT_EXT);
		//JointMap.Add(,	XR_HAND_JOINT_PALM_EXT); // ARKit has no palm, we will synthesize one.
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_wrist,	XR_HAND_JOINT_WRIST_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_thumb_knuckle,	XR_HAND_JOINT_THUMB_METACARPAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_thumb_intermediate_base,	XR_HAND_JOINT_THUMB_PROXIMAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_thumb_intermediate_tip,	XR_HAND_JOINT_THUMB_DISTAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_thumb_tip,	XR_HAND_JOINT_THUMB_TIP_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_index_finger_metacarpal,	XR_HAND_JOINT_INDEX_METACARPAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_index_finger_knuckle,	XR_HAND_JOINT_INDEX_PROXIMAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_index_finger_intermediate_base,	XR_HAND_JOINT_INDEX_INTERMEDIATE_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_index_finger_intermediate_tip,	XR_HAND_JOINT_INDEX_DISTAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_index_finger_tip,	XR_HAND_JOINT_INDEX_TIP_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_middle_finger_metacarpal,	XR_HAND_JOINT_MIDDLE_METACARPAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_middle_finger_knuckle,	XR_HAND_JOINT_MIDDLE_PROXIMAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_middle_finger_intermediate_base,	XR_HAND_JOINT_MIDDLE_INTERMEDIATE_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_middle_finger_intermediate_tip,	XR_HAND_JOINT_MIDDLE_DISTAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_middle_finger_tip,	XR_HAND_JOINT_MIDDLE_TIP_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_ring_finger_metacarpal,	XR_HAND_JOINT_RING_METACARPAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_ring_finger_knuckle,	XR_HAND_JOINT_RING_PROXIMAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_ring_finger_intermediate_base,	XR_HAND_JOINT_RING_INTERMEDIATE_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_ring_finger_intermediate_tip,	XR_HAND_JOINT_RING_DISTAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_ring_finger_tip,	XR_HAND_JOINT_RING_TIP_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_little_finger_metacarpal,				XR_HAND_JOINT_LITTLE_METACARPAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_little_finger_knuckle,					XR_HAND_JOINT_LITTLE_PROXIMAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_little_finger_intermediate_base,		XR_HAND_JOINT_LITTLE_INTERMEDIATE_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_little_finger_intermediate_tip,		XR_HAND_JOINT_LITTLE_DISTAL_EXT));
		JointMap.Add(TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>(ar_hand_skeleton_joint_name_little_finger_tip,						XR_HAND_JOINT_LITTLE_TIP_EXT));
		//ar_hand_skeleton_joint_name_forearm_arm  // OpenXR does not support these forearm joints, there is an extension to support an elbow, we could add support for that.
		//ar_hand_skeleton_joint_name_forearm_wrist
	}
	
	FOXRVisionOSHandTracker& LeftHandTracker = GetHandTracker(XR_HAND_LEFT_EXT);
	FOXRVisionOSHandTracker& RightHandTracker = GetHandTracker(XR_HAND_RIGHT_EXT);
	LeftHandTracker.Anchor = ar_hand_anchor_create();
	RightHandTracker.Anchor = ar_hand_anchor_create();
}

void FOXRVisionOSSession::SyncHandTracking()
{
	// We will retain the joint local transforms and the hand transforms so that partial updates can be applied.
	
	FOXRVisionOSHandTracker& LeftHandTracker = GetHandTracker(XR_HAND_LEFT_EXT);
	FOXRVisionOSHandTracker& RightHandTracker = GetHandTracker(XR_HAND_RIGHT_EXT);
	
	bool Success = ar_hand_tracking_provider_get_latest_anchors(ARKitHandTrackingProvider, LeftHandTracker.Anchor, RightHandTracker.Anchor);
	if (Success == false) 
	{
		return;
	}

	auto SyncHand = []( FOXRVisionOSHandTracker& HandTracker, const TArray<TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>>& TheJointMap, bool bIsLeft)
	{
		HandTracker.bIsActive = true;
		
		bool bHandTracked = ar_trackable_anchor_is_tracked(HandTracker.Anchor);
		if (bHandTracked) 
		{
			HandTracker.HandTransform = ar_anchor_get_origin_from_anchor_transform(HandTracker.Anchor);
		}
		
		// The left hand's joints are all flipped vs the openxr spec, we need to rotate them 180 degrees around z.
		const simd_float4x4 Transform180Z = simd_diagonal_matrix(simd_make_float4(-1,-1,1,1));

		ar_hand_skeleton_t Skeleton = ar_hand_anchor_get_hand_skeleton(HandTracker.Anchor);
		for(auto& Pair : TheJointMap)
		{
			ar_skeleton_joint_t Joint = ar_hand_skeleton_get_joint_named(Skeleton, Pair.Key);
			simd_float4x4& JointLocalTransform = HandTracker.JointLocalTransforms[Pair.Value];
			XrHandJointLocationEXT& JointLocation = HandTracker.JointLocations[Pair.Value];
			
			bool bIsTracked = ar_skeleton_joint_is_tracked(Joint);
			if (bIsTracked) 
			{
				JointLocation.locationFlags = XR_SPACE_LOCATION_POSITION_VALID_BIT | XR_SPACE_LOCATION_ORIENTATION_VALID_BIT | XR_SPACE_LOCATION_POSITION_TRACKED_BIT | XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT;
			}
			else 
			{
				// Clear the tracked bits, but leave the valid bits and transforms as they are.  This means the joint is valid when it has been tracked once.
				JointLocation.locationFlags = JointLocation.locationFlags & (XR_SPACE_LOCATION_POSITION_VALID_BIT | XR_SPACE_LOCATION_ORIENTATION_VALID_BIT);
			}
			JointLocalTransform = ar_skeleton_joint_get_anchor_from_joint_transform(Joint); // Even untracked joints do get updates from their parent's positions.

			// We udpate the tracking space transform regardless of tracking status.  If either the local transform or the hand transform have updated it is useful.  If neither it doesn't really hurt.
			simd_float4x4 JointTrackingTransform = matrix_multiply(HandTracker.HandTransform, JointLocalTransform);
			if (bIsLeft) {
				JointTrackingTransform = matrix_multiply(JointTrackingTransform, Transform180Z);
			}
			JointLocation.pose = OXRVisionOS::ToXrPose(JointTrackingTransform);
			JointLocation.radius = 0.005f;  // In Meters. We could estimate this from the distance between joints and some allometric data about humans...
		}
		
		// Special case for the palm, which ARKit does not provide.
		{
			//We will take the Middle Finger Metacarpal as the rotation of the Palm.
			//We will average the Middle Finger Proximal and Middle Finger Metacarpal as the position of the Palm.
			//The palm's flags will be the & of the two other joints flags.
			//If one of the other bones is tracked and the other is not we might get an odd transform for the palm, but it will not be flagged as tracked.
			simd_float4x4& PalmJointLocalTransform = HandTracker.JointLocalTransforms[XR_HAND_JOINT_PALM_EXT];
			const simd_float4x4& MiddleProximalJointLocalTransform = HandTracker.JointLocalTransforms[XR_HAND_JOINT_MIDDLE_PROXIMAL_EXT];
			const simd_float4x4& MiddleMetacarpalJointLocalTransform = HandTracker.JointLocalTransforms[XR_HAND_JOINT_MIDDLE_METACARPAL_EXT];
			PalmJointLocalTransform = MiddleMetacarpalJointLocalTransform;
			PalmJointLocalTransform.columns[3][0] = (MiddleProximalJointLocalTransform.columns[3][0] + MiddleMetacarpalJointLocalTransform.columns[3][0]) * 0.5f;
			PalmJointLocalTransform.columns[3][1] = (MiddleProximalJointLocalTransform.columns[3][1] + MiddleMetacarpalJointLocalTransform.columns[3][1]) * 0.5f;
			PalmJointLocalTransform.columns[3][2] = (MiddleProximalJointLocalTransform.columns[3][2] + MiddleMetacarpalJointLocalTransform.columns[3][2]) * 0.5f;
			
			simd_float4x4 JointTrackingTransform = matrix_multiply(HandTracker.HandTransform, PalmJointLocalTransform);
			if (bIsLeft) {
				JointTrackingTransform = matrix_multiply(JointTrackingTransform, Transform180Z);
			}
			
			XrHandJointLocationEXT& PalmJoint = HandTracker.JointLocations[XR_HAND_JOINT_PALM_EXT];
			XrHandJointLocationEXT& MiddleProximalJoint = HandTracker.JointLocations[XR_HAND_JOINT_MIDDLE_PROXIMAL_EXT];
			XrHandJointLocationEXT& MiddleMetacarpalJoint = HandTracker.JointLocations[XR_HAND_JOINT_MIDDLE_METACARPAL_EXT];
			PalmJoint.locationFlags = MiddleProximalJoint.locationFlags & MiddleMetacarpalJoint.locationFlags;
			PalmJoint.pose = OXRVisionOS::ToXrPose(JointTrackingTransform);
			PalmJoint.radius = 0.005f;  // In Meters. We could estimate this from the distance between joints and some allometric data about humans...

		}
	};
	
	SyncHand(LeftHandTracker, JointMap, true);
	SyncHand(RightHandTracker, JointMap, false);
}


