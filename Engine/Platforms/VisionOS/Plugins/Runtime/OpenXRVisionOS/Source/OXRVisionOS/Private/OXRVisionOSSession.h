// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OXRVisionOS.h"

#include "Engine/EngineBaseTypes.h"
#include "Epic_openxr.h"
#include "OXRVisionOS_openxr_platform.h"
#include "HAL/Runnable.h"
#include "HeadMountedDisplayTypes.h"

#if PLATFORM_VISIONOS
#import <ARKit/ARKit.h>
#import <CompositorServices/CompositorServices.h>
#endif


class FOXRVisionOSInstance;
class FOXRVisionOSActionSet;
class FOXRVisionOSSpace;
class FOXRVisionOSSwapchain;
class FOXRVisionOSTracker;
class FOXRVisionOSController;

class FRHICommandListImmediate;
enum class EOXRVisionOSControllerButton : int32;

class FOXRVisionOSSession
{
public:
	static XrResult Create(TSharedPtr<FOXRVisionOSSession, ESPMode::ThreadSafe>& OutSession, const XrSessionCreateInfo* createInfo, FOXRVisionOSInstance* Instance);
	FOXRVisionOSSession(const XrSessionCreateInfo* createInfo, FOXRVisionOSInstance* Instance);
	~FOXRVisionOSSession();
	XrResult XrDestroySession();

	XrResult XrEnumerateReferenceSpaces(
		uint32_t spaceCapacityInput,
		uint32_t* spaceCountOutput,
		XrReferenceSpaceType* spaces);

	XrResult XrCreateReferenceSpace(
		const XrReferenceSpaceCreateInfo* createInfo,
		XrSpace* space);

	XrResult XrGetReferenceSpaceBoundsRect(
		XrReferenceSpaceType referenceSpaceType,
		XrExtent2Df* bounds);

	XrResult XrCreateActionSpace(
		const XrActionSpaceCreateInfo* createInfo,
		XrSpace* space);
	
	const FOXRVisionOSSpace* ToFOXRVisionOSSpace(const XrSpace& Space);

	XrResult DestroySpace(
		FOXRVisionOSSpace* Space);

	XrResult XrEnumerateSwapchainFormats(
		uint32_t formatCapacityInput,
		uint32_t* formatCountOutput,
		int64_t* formats);

	XrResult XrCreateSwapchain(
		const XrSwapchainCreateInfo* createInfo,
		XrSwapchain* swapchain);

	XrResult DestroySwapchain(
		FOXRVisionOSSwapchain* Swapchain);

	XrResult XrAttachSessionActionSets(
		const XrSessionActionSetsAttachInfo* attachInfo);

	XrResult XrGetCurrentInteractionProfile(
		XrPath topLevelUserPath,
		XrInteractionProfileState* interactionProfile);

	XrResult XrGetActionStateBoolean(
		const XrActionStateGetInfo* getInfo,
		XrActionStateBoolean* state);

	XrResult XrGetActionStateFloat(
		const XrActionStateGetInfo* getInfo,
		XrActionStateFloat* state);

	XrResult XrGetActionStateVector2f(
		const XrActionStateGetInfo* getInfo,
		XrActionStateVector2f* state);

	XrResult XrGetActionStatePose(
		const XrActionStateGetInfo* getInfo,
		XrActionStatePose* state);

	XrResult XrSyncActions(
		const XrActionsSyncInfo* syncInfo);

	XrResult XrEnumerateBoundSourcesForAction(
		const XrBoundSourcesForActionEnumerateInfo* enumerateInfo,
		uint32_t sourceCapacityInput,
		uint32_t* sourceCountOutput,
		XrPath* sources);

	XrResult XrApplyHapticFeedback(
		const XrHapticActionInfo* hapticActionInfo,
		const XrHapticBaseHeader* hapticFeedback);

	XrResult XrStopHapticFeedback(
		const XrHapticActionInfo* hapticActionInfo);

	XrResult XrBeginSession(
		const XrSessionBeginInfo* beginInfo);

	XrResult XrEndSession();

	void EndSessionInternal();

	XrResult XrRequestExitSession();

	XrResult XrWaitFrame(
		const XrFrameWaitInfo* frameWaitInfo,
		XrFrameState* frameState);

	void OnBeginRendering_GameThread();

	void OnBeginRendering_RenderThread_PreDeviceLocationUpdate();

	void OnBeginFrame_RHIThread(FRHICommandListImmediate* InRHICommandList);

	XrResult XrBeginFrame(
		const XrFrameBeginInfo* frameBeginInfo);

	XrResult XrEndFrame(
		const XrFrameEndInfo* frameEndInfo);

	XrResult XrLocateViews(
		const XrViewLocateInfo* viewLocateInfo,
		XrViewState* viewState,
		uint32_t viewCapacityInput,
		uint32_t* viewCountOutput,
		XrView* views);
	
	XrResult XrCreateHandTrackerEXT(
		const XrHandTrackerCreateInfoEXT*           createInfo,
		XrHandTrackerEXT*                           handTracker);

	XrTime GetCurrentTime() const;

	void SetSessionState(XrSessionState NewState);
	XrSessionState GetSessionState() const { return SessionState; }

	bool IsSessionRunning() const;

	FOXRVisionOSTracker& GetTrackerChecked() const { check(Tracker.IsValid()); return *Tracker; }

	struct FPipelinedFrameState
	{
		int32 FrameCounter = -1;
		XrTime PredictedDisplayTime = 0;
		//FTransform HMDPoseInTrackerSpace = FTransform::Identity;
		//PoseData HmdPose = {};
		cp_frame_t SwiftFrame = nullptr;
		cp_frame_timing_t SwiftFrameTiming = nullptr;  // Used for the cp_time_wait_until
		CFTimeInterval PreviousPredictionTime = 0;
		cp_drawable_t SwiftDrawable = nullptr;
        cp_frame_timing_t SwiftFinalFrameTiming = nullptr; // TODO should we do this or should we overwrite SwifFrameTiming???
		simd_float4x4 HeadTransform = matrix_identity_float4x4;
		CFTimeInterval SwiftFinalFrameTimeInterval = 0;
        ar_device_anchor_t DeviceAnchor = nullptr;
		bool bSynchronizing = true; // when true we are in the process of starting up a new session and the new frame state has not yet propagated.
		bool bShouldRender = false;
		bool bSessionLost = false;
		XrFovf HmdFovs[2];
        int32 LocateViewInfoBufferIndex = 0;
		int32 RenderToGameFrameStateIndex = -1;
	};
	
	struct FRenderToGameFrameState
	{
		simd_float4x4 HeadTransform = matrix_identity_float4x4;
		CFTimeInterval PredictionTime = 0;
	};
	void RenderToGameFrameStateRead(FPipelinedFrameState& InOutFrameState);
	void RenderToGameFrameStateWrite(const FPipelinedFrameState& FrameState);

	const FPipelinedFrameState& GetPipelinedFrameStateForThread() const;
	FPipelinedFrameState& GetPipelinedFrameStateForThread();
	void GetHMDTransform(XrTime DisplayTime, FTransform& OutTransform, XrSpaceLocationFlags& OutLocationFlags);
	void GetTransformForButton(EOXRVisionOSControllerButton Button, XrTime DisplayTime, FTransform& OutTransform, XrSpaceLocationFlags& OutLocationFlags, FVector& OutLinearVelocity, FVector& OutAngularVelocity, XrSpaceVelocityFlags& OutVelocityFlags);
	void SetTriggerEffectVRControllers(int32 ControllerId, const FInputDeviceProperty* Property);
	// FOpenXRInput::Tick, game thread, after xrSyncActions
	void OnPostSyncActions(); 

	struct FLocateViewInfo
    {
        uint32_t NumViews = 0;
        simd_float4x4 ViewTransforms[2];
        XrFovf HmdFovs[2];
    };
    static constexpr int LocateViewInfoBufferLength = 3;
    FLocateViewInfo LocateViewInfoBuffer[LocateViewInfoBufferLength];
    void LocateViewInfoIndexAdvance() { ++PipelinedFrameStateGame.LocateViewInfoBufferIndex; }
    const FLocateViewInfo& GetLocateViewInfo_GameThread() { return LocateViewInfoBuffer[(PipelinedFrameStateGame.LocateViewInfoBufferIndex - 1) % LocateViewInfoBufferLength]; }
    FLocateViewInfo& GetLocateViewInfo_RenderThread() { return LocateViewInfoBuffer[PipelinedFrameStateRendering.LocateViewInfoBufferIndex % LocateViewInfoBufferLength]; }
	
private:
	bool bCreateFailed = false;
	bool bIsRunning = false;
	bool bExitRequested = false;
	FOXRVisionOSInstance* Instance = nullptr;
	uint64_t XRTimeScale = 1;
	XrSessionState SessionState = XR_SESSION_STATE_UNKNOWN;
	bool ActionSetsAlreadyAttached = false;
	TArray<TSharedPtr<FOXRVisionOSActionSet, ESPMode::ThreadSafe>> ActionSets;
	TSharedPtr<FOXRVisionOSController, ESPMode::ThreadSafe> Controllers;
	TArray<TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe>> Spaces;
	FQuat GripPoseAdjustment;
	FQuat AimPoseAdjustment;
	FQuat GripSurfacePoseAdjustment;

	FPlatformMemory::FPlatformVirtualMemoryBlock m_reprojectionData;
	FPlatformMemory::FPlatformVirtualMemoryBlock m_displayData;

	TArray<TSharedPtr<FOXRVisionOSSwapchain, ESPMode::ThreadSafe>> Swapchains;

	int32 NextBackBufferIndex = 0;
	int32 PreviousBackBufferIndexInBegin = -1;
	int32 SessionFrameCounter = 0;

	static const int32 INVALID_DEVICE_HANDLE = -1;
	int32 HMDHandle = INVALID_DEVICE_HANDLE;

	TSharedPtr <FOXRVisionOSTracker, ESPMode::ThreadSafe> Tracker;

	//TEMP
	static const int32 RESULT_OK = 0;


	void OnWorldTickStart(UWorld* World, ELevelTick TickType, float DeltaTime);
	FDelegateHandle OnWorldTickStartDelegateHandle;

	void WaitUntil();
    void StartSubmission();
	FEvent* XrWaitFrameEvent = nullptr;
	FPipelinedFrameState	PipelinedFrameStateGame;
	FPipelinedFrameState	PipelinedFrameStateRendering;
	FPipelinedFrameState	PipelinedFrameStateRHI;
	int32 CachedBeginFlipFrameCounter = INT32_MAX;
	
	static const int32 RenderToGameFrameStateBufferLength = 3;
	FRenderToGameFrameState RenderToGameFrameState[RenderToGameFrameStateBufferLength];

	FOpenXRRenderBridge* RenderBridge = nullptr;

	CP_OBJECT_cp_layer_renderer* VOSLayerRenderer = nullptr;
	ar_session_t ARKitSession = nullptr;
	ar_world_tracking_provider_t ARKitWorldTrackingProvider = nullptr;
	ar_device_anchor_t ARKitHMDAnchor = nullptr;
	
	// mutexes to protect apple api functions from concurrent calls
	// We do not protect the early startup calls, before OXRVisionOSSession is involved.  They should be single threaded.
	mutable FCriticalSection CpLayerMutex; // Protect all cp_label_ calls.
	//mutable FCriticalSection CpFrameMutex; // Currently we are restricting cp_frame_ calls to the Render thread by not having an RHI thread, however we may want to change that in the future...
	
	// Hand Tracking
public:
	struct FOXRVisionOSHandTracker
	{
		FOXRVisionOSHandTracker()
		{
			HandTransform = matrix_identity_float4x4;
			for (int i = 0; i < XR_HAND_JOINT_COUNT_EXT; ++i)
			{
				JointLocations[i].pose.orientation.w = 1;
				JointLocalTransforms[i] = matrix_identity_float4x4;
			}
		}
		XrResult XrDestroyHandTrackerEXT();

		XrResult XrLocateHandJointsEXT(
			const XrHandJointsLocateInfoEXT*            locateInfo,
			XrHandJointLocationsEXT*                    locations);
		
		bool bCreated = false;
		bool bIsActive = false;
		ar_hand_anchor_t Anchor;
		simd_float4x4 HandTransform;
		simd_float4x4 JointLocalTransforms[XR_HAND_JOINT_COUNT_EXT];
		XrHandJointLocationEXT JointLocations[XR_HAND_JOINT_COUNT_EXT] = {};
	};
private:
	void BeginHandTracking();
	ar_hand_tracking_provider_t ARKitHandTrackingProvider = nullptr;
	TArray<TPair<ar_hand_skeleton_joint_name_t,XrHandJointEXT>> JointMap;
	FOXRVisionOSHandTracker HandTrackers[2];
	static_assert(XR_HAND_LEFT_EXT == 1 && XR_HAND_RIGHT_EXT == 2);  // We will assume this to use value-1 as array index.
	FOXRVisionOSHandTracker& GetHandTracker(XrHandEXT Hand) { return HandTrackers[Hand - 1]; }
	const FOXRVisionOSHandTracker& GetHandTracker(XrHandEXT Hand) const { return HandTrackers[Hand - 1]; }
	bool ValidateXrHandTrackerEXT(XrHandTrackerEXT HandTracker) { return (((FOXRVisionOSHandTracker*)HandTracker) - ((FOXRVisionOSHandTracker*)&HandTrackers)) <= 1; }
	void SyncHandTracking();
};
