// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSSpace.h"
#include "OXRVisionOSSession.h"
#include "OXRVisionOSPlatformUtils.h"
#include "OXRVisionOSTracker.h"
#include "OXRVisionOSAction.h"
#include "OXRVisionOSController.h"

//#include "../../../../../../../../Source/ThirdParty/OpenXR/include/openxr/openxr_platform.h"
//#include <openxr/openxr_platform.h>

XrResult FOXRVisionOSSpace::CreateActionSpace(TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe>& OutSpace, const XrActionSpaceCreateInfo* CreateInfo, FOXRVisionOSSession* Session)
{
	if (CreateInfo == nullptr || Session == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (CreateInfo->type != XR_TYPE_ACTION_SPACE_CREATE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (CreateInfo->action == XR_NULL_HANDLE)
	{
		return XrResult::XR_ERROR_HANDLE_INVALID;
	}
	
	const FOXRVisionOSAction& Action = *((FOXRVisionOSAction*)CreateInfo->action);
	if (Action.GetActionType() != XR_ACTION_TYPE_POSE_INPUT)
	{
		return XR_ERROR_ACTION_TYPE_MISMATCH;
	}

	OutSpace = MakeShared<FOXRVisionOSSpace, ESPMode::ThreadSafe>(CreateInfo, Session);
	if (OutSpace->bCreateFailed)
	{
		OutSpace = nullptr;
		return XrResult::XR_ERROR_RUNTIME_FAILURE;
	}

	return XrResult::XR_SUCCESS;
}

FOXRVisionOSSpace::FOXRVisionOSSpace(const XrActionSpaceCreateInfo* CreateInfo, FOXRVisionOSSession* InSession)
{
	Session = InSession;

	bIsReferenceSpace = false;
	Action = CreateInfo->action;
	SubactionPath = CreateInfo->subactionPath;
	Pose = CreateInfo->poseInActionSpace;
}

XrResult FOXRVisionOSSpace::CreateReferenceSpace(TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe>& OutSpace, const XrReferenceSpaceCreateInfo* CreateInfo, FOXRVisionOSSession* Session)
{
	if (CreateInfo == nullptr || Session == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (CreateInfo->type != XR_TYPE_REFERENCE_SPACE_CREATE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	OutSpace = MakeShared<FOXRVisionOSSpace, ESPMode::ThreadSafe>(CreateInfo, Session);
	if (OutSpace->bCreateFailed)
	{
		OutSpace = nullptr;
		return XrResult::XR_ERROR_RUNTIME_FAILURE;
	}

	return XrResult::XR_SUCCESS;
}

FOXRVisionOSSpace::FOXRVisionOSSpace(const XrReferenceSpaceCreateInfo* CreateInfo, FOXRVisionOSSession* InSession)
{
	Session = InSession;

	bIsReferenceSpace = true;
	ReferenceSpaceType = CreateInfo->referenceSpaceType;
	Pose = CreateInfo->poseInReferenceSpace;
	UEPose = ToFTransform(Pose);
	UEPoseInverse = UEPose.Inverse();
}

FOXRVisionOSSpace::~FOXRVisionOSSpace()
{
	if (bCreateFailed)
	{
		UE_LOGF(LogOXRVisionOS, Warning, "Destructing FOXRVisionOSSpace because create failed.");
	}
}

XrResult FOXRVisionOSSpace::XrDestroySpace()
{
	// Session->DestroySpace() can delete this, so better just return after that.
	return Session->DestroySpace(this);
}

XrResult FOXRVisionOSSpace::XrLocateSpace(
	XrSpace                                     InBaseSpace,
	XrTime                                      Time,
	XrSpaceLocation*							Location) 
{
	if (InBaseSpace == XR_NULL_HANDLE)
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	XrSpaceVelocity* Velocity = OpenXR::FindChainedStructByType<XrSpaceVelocity>(Location->next, (XrStructureType)XR_TYPE_SPACE_VELOCITY);

	// When not focused we do not return any tracking data.
	if (Session->GetSessionState() != XrSessionState::XR_SESSION_STATE_FOCUSED)
	{
		Location->locationFlags = 0;
		if (Velocity)
		{
			Velocity->velocityFlags = 0;
		}

		return XrResult::XR_SUCCESS;
	}

	if (InBaseSpace == (XrSpace)this)
	{
		// locate vs self is always identity
		Location->locationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT
			| XR_SPACE_LOCATION_POSITION_VALID_BIT
			| XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT
			| XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
		Location->pose = XrPosefIdentity;

		if (Velocity)
		{
			// velocity vs self is always zero
			Velocity->velocityFlags = XR_SPACE_VELOCITY_LINEAR_VALID_BIT | XR_SPACE_VELOCITY_ANGULAR_VALID_BIT;
			Velocity->linearVelocity = OXRVisionOS::XrZeroVector;
			Velocity->angularVelocity = OXRVisionOS::XrZeroVector;
		}

	}

	FOXRVisionOSSpace& BaseSpace = *(FOXRVisionOSSpace*)InBaseSpace;

	if (IsSpaceRelativeToSameEntity(BaseSpace))
	{
		// locate vs a space which is relative to the same entity is always valid and tracked and determined only by their poses relative to that entity.
		Location->locationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT
			| XR_SPACE_LOCATION_POSITION_VALID_BIT
			| XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT
			| XR_SPACE_LOCATION_POSITION_TRACKED_BIT;

		FTransform Result = UEPose * BaseSpace.UEPoseInverse;
		Location->pose = ToXrPose(Result);

		if (Velocity)
		{
			// velocity vs a sibling attached to the same parent is always zero
			Velocity->velocityFlags = XR_SPACE_VELOCITY_LINEAR_VALID_BIT | XR_SPACE_VELOCITY_ANGULAR_VALID_BIT;
			Velocity->linearVelocity = OXRVisionOS::XrZeroVector;
			Velocity->angularVelocity = OXRVisionOS::XrZeroVector;
		}

		return XrResult::XR_SUCCESS;
	}

	FTransform MyTransform;
	XrSpaceLocationFlags MyFlags = 0;
	FVector MyLinearVelocity;
	FVector MyAngularVelocity;
	XrSpaceVelocityFlags MyVelocityFlags = 0;

	FTransform TheirInverseTransform;
	XrSpaceLocationFlags TheirFlags = 0;
	FVector TheirInverseLinearVelocity;
	FVector TheirInverseAngularVelocity;
	XrSpaceVelocityFlags TheirVelocityFlags = 0;

	GetTransform(Time, MyTransform, MyFlags, MyLinearVelocity, MyAngularVelocity, MyVelocityFlags);
	BaseSpace.GetInverseTransform(Time, TheirInverseTransform, TheirFlags, TheirInverseLinearVelocity, TheirInverseAngularVelocity, TheirVelocityFlags);
	FTransform Result = MyTransform * TheirInverseTransform;
	Location->pose = ToXrPose(Result);

	Location->locationFlags = MyFlags & TheirFlags;

	if (Velocity)
	{
		Velocity->velocityFlags = 0;
		if ((Location->locationFlags & XR_SPACE_LOCATION_ORIENTATION_VALID_BIT) && (MyVelocityFlags & XR_SPACE_VELOCITY_LINEAR_VALID_BIT) && (TheirVelocityFlags & XR_SPACE_VELOCITY_LINEAR_VALID_BIT))
		{
			FVector RelativeVelocity = MyLinearVelocity + TheirInverseLinearVelocity;
			Velocity->linearVelocity = OXRVisionOS::ToXrVector3f(TheirInverseTransform.TransformVectorNoScale(RelativeVelocity));
			Velocity->velocityFlags |= XR_SPACE_VELOCITY_LINEAR_VALID_BIT;
		}
		if ((Location->locationFlags & XR_SPACE_LOCATION_ORIENTATION_VALID_BIT) && (MyVelocityFlags & XR_SPACE_VELOCITY_ANGULAR_VALID_BIT) && (TheirVelocityFlags & XR_SPACE_VELOCITY_ANGULAR_VALID_BIT))
		{
			FVector RelativeAngularVelocity = MyAngularVelocity + TheirInverseAngularVelocity;
			Velocity->angularVelocity = OXRVisionOS::ToXrVector3f(TheirInverseTransform.TransformVectorNoScale(RelativeAngularVelocity));
			Velocity->velocityFlags |= XR_SPACE_VELOCITY_ANGULAR_VALID_BIT;
		}
	}

	return XrResult::XR_SUCCESS;
}

bool FOXRVisionOSSpace::IsSpaceRelativeToSameEntity(FOXRVisionOSSpace& OtherSpace) const
{
	if (bIsReferenceSpace != OtherSpace.bIsReferenceSpace)
	{
		return false;
	}

	if (bIsReferenceSpace)
	{
		return ReferenceSpaceType == OtherSpace.ReferenceSpaceType;
	}
	else
	{
		//TODO is this correct??
		return Action == OtherSpace.Action;
	}
}
void FOXRVisionOSSpace::GetTransform(XrTime DisplayTime, FTransform& OutTransform, XrSpaceLocationFlags& OutLocationFlags, FVector& OutLinearVelocity, FVector& OutAngularVelocity, XrSpaceVelocityFlags& OutVelocityFlags)
{
	OutLocationFlags = 0;

	if (bIsReferenceSpace)
	{
		switch (ReferenceSpaceType)
		{
		case XR_REFERENCE_SPACE_TYPE_VIEW:
		{
			// The view is the hmd.
            // Local space is tracking space, so it is always at the origin + any pose it was given and velocity zero.
            OutLocationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT
                | XR_SPACE_LOCATION_POSITION_VALID_BIT
                | XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT
                | XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
            Session->GetHMDTransform(DisplayTime, OutTransform, OutLocationFlags);
            OutVelocityFlags = 0;
            OutLinearVelocity = FVector::ZeroVector;
            OutAngularVelocity = FVector::ZeroVector;
            return;
		}
		case XR_REFERENCE_SPACE_TYPE_LOCAL:
		{
			// Local space is tracking space, so it is always at the origin + any pose it was given and velocity zero.
			OutLocationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT
				| XR_SPACE_LOCATION_POSITION_VALID_BIT
				| XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT
				| XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
			OutTransform = UEPose;
			OutVelocityFlags = XR_SPACE_VELOCITY_LINEAR_VALID_BIT | XR_SPACE_VELOCITY_ANGULAR_VALID_BIT;
			OutLinearVelocity = FVector::ZeroVector;
			OutAngularVelocity = FVector::ZeroVector;
			return;
		}
		case XR_REFERENCE_SPACE_TYPE_LOCAL_FLOOR:
		{
			// Local floor space is tracking space, so it is always at the origin + any pose it was given and velocity zero.
			OutLocationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT
				| XR_SPACE_LOCATION_POSITION_VALID_BIT
				| XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT
				| XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
			OutTransform = UEPose;
			OutVelocityFlags = XR_SPACE_VELOCITY_LINEAR_VALID_BIT | XR_SPACE_VELOCITY_ANGULAR_VALID_BIT;
			OutLinearVelocity = FVector::ZeroVector;
			OutAngularVelocity = FVector::ZeroVector;
			return;
		}
//		case XR_REFERENCE_SPACE_TYPE_STAGE:
//		{
//			// The stage is the space of the reference space, aka the playable area.  See also XrGetReferenceSpaceBoundsRect.
//
//			FOXRVisionOSTracker& Tracker = Session->GetTrackerChecked();
//			OXRVisionOS::TrackerPoseData PoseData = {};
//			bool bSuccess = Tracker.GetStagePose(PoseData);
//			if (bSuccess)
//			{
//				OutLocationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT
//					| XR_SPACE_LOCATION_POSITION_VALID_BIT
//					| XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT
//					| XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
//				FQuat OriginQuat = OXRVisionOS::ToFQuat(PoseData.orientation);
//				FVector OriginVector = OXRVisionOS::ToFVector(PoseData.position);
//				OutTransform = FTransform(OriginQuat, OriginVector) * UEPose;
//				OutVelocityFlags = XR_SPACE_VELOCITY_LINEAR_VALID_BIT | XR_SPACE_VELOCITY_ANGULAR_VALID_BIT;
//				OutLinearVelocity = FVector::ZeroVector;
//				OutAngularVelocity = FVector::ZeroVector;				
//			}
//			return;
//		}			
		default:
			// Shouldn't be possible to create an unsupported reference space.
			check(false);
			return;
		}
	}
	else
	{
		// Not a reference space, so an action space
		check(Action != XR_NULL_HANDLE);

		FOXRVisionOSAction* TheAction = (FOXRVisionOSAction*)Action;

		// figure out what transform to get
		EOXRVisionOSControllerButton Button = TheAction->GetPoseButton(SubactionPath);
		Session->GetTransformForButton(Button, DisplayTime, OutTransform, OutLocationFlags, OutLinearVelocity, OutAngularVelocity, OutVelocityFlags);
	}
}

void FOXRVisionOSSpace::GetInverseTransform(XrTime Time, FTransform& OutTransform, XrSpaceLocationFlags& OutLocationFlags, FVector& OutLinearVelocity, FVector& OutAngularVelocity, XrSpaceVelocityFlags& OutVelocityFlags)
{
	if (bIsReferenceSpace && (ReferenceSpaceType == XR_REFERENCE_SPACE_TYPE_LOCAL))
	{
		// Local space is tracking space, so it is always at the origin + any pose it was given and velocity zero.
		OutLocationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT
			| XR_SPACE_LOCATION_POSITION_VALID_BIT
			| XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT
			| XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
		OutTransform = UEPoseInverse;
		OutVelocityFlags = XR_SPACE_VELOCITY_LINEAR_VALID_BIT | XR_SPACE_VELOCITY_ANGULAR_VALID_BIT;
		OutLinearVelocity = FVector::ZeroVector;
		OutAngularVelocity = FVector::ZeroVector;
	}
	else
	{
		GetTransform(Time, OutTransform, OutLocationFlags, OutLinearVelocity, OutAngularVelocity, OutVelocityFlags);
		OutTransform = OutTransform.Inverse();
		if (OutVelocityFlags & XR_SPACE_VELOCITY_LINEAR_VALID_BIT)
		{
			OutLinearVelocity = -OutLinearVelocity;
		}
		if (OutVelocityFlags & XR_SPACE_VELOCITY_LINEAR_VALID_BIT)
		{
			OutAngularVelocity = -OutAngularVelocity;
		}
	}
}
