// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OXRVisionOS.h"
#include "OXRVisionOS_openxr_platform.h"
#include "Epic_openxr.h"

class FOXRVisionOSSession;

class FOXRVisionOSSpace
{
public:
	static XrResult CreateActionSpace(TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe>& OutSpace, const XrActionSpaceCreateInfo* createInfo, FOXRVisionOSSession* Session);
	FOXRVisionOSSpace(const XrActionSpaceCreateInfo* createInfo, FOXRVisionOSSession* Session);
	static XrResult CreateReferenceSpace(TSharedPtr<FOXRVisionOSSpace, ESPMode::ThreadSafe>& OutSpace, const XrReferenceSpaceCreateInfo* createInfo, FOXRVisionOSSession* Session);
	FOXRVisionOSSpace(const XrReferenceSpaceCreateInfo* createInfo, FOXRVisionOSSession* Session);
	~FOXRVisionOSSpace();
	XrResult XrDestroySpace();

	XrResult XrLocateSpace(
		XrSpace                                     baseSpace,
		XrTime                                      time,
		XrSpaceLocation* location);
	void GetTransform(XrTime Time, FTransform& OutTransform, XrSpaceLocationFlags& OutLocationFlags, FVector& OutLinearVelocity, FVector& OutAngularVelocity, XrSpaceVelocityFlags& OutVelocityFlags);
	void GetInverseTransform(XrTime DisplayTime, FTransform& OutTransform, XrSpaceLocationFlags& OutLocationFlags, FVector& OutLinearVelocity, FVector& OutAngularVelocity, XrSpaceVelocityFlags& OutVelocityFlags);

	bool IsReferenceSpace() const { return bIsReferenceSpace; }
	XrReferenceSpaceType GetReferenceSpaceType() const
	{
		if (bIsReferenceSpace)
		{
			return ReferenceSpaceType;
		}
		return XR_REFERENCE_SPACE_TYPE_MAX_ENUM;
	}

private:
	const XrPosef XrPosefIdentity = { { 0.0f, 0.0f, 0.0f, 1.0f }, { 0.0f, 0.0f, 0.0f } };

	bool IsSpaceRelativeToSameEntity(FOXRVisionOSSpace& OtherSpace) const;

	bool bCreateFailed = false;
	FOXRVisionOSSession* Session = nullptr;

	// EITHER a reference space
	bool bIsReferenceSpace = false;
	XrReferenceSpaceType ReferenceSpaceType;
	// OR an action space
	XrAction Action = XR_NULL_HANDLE;
	XrPath SubactionPath = XR_NULL_PATH;

	//TODO support recenter past timestamps for 50ms
	XrPosef Pose;
	FTransform UEPose;
	FTransform UEPoseInverse;
};
