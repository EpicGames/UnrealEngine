// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSSwapchain.h"

#if PLATFORM_MAC
#include "Mac/CocoaTextView.h"
@interface FMetalView : FCocoaTextView
@end
#endif
#include "HAL/PlatformFramePacer.h"

#if PLATFORM_VISIONOS
#import <CompositorServices/CompositorServices.h>
#endif

#include "OXRVisionOSSession.h"
#include "OXRVisionOS_openxr_platform.h"
#include "RHICommandList.h"
//#include "MetalRHIOXRVisionOSBridge.h"

// uncomment to make intellisense work better
//#include "../../../../../../../../Source/ThirdParty/OpenXR/include/openxr/openxr_platform.h"

static int GetMSAASampleCount()
{
	static int Mode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.XRMSAAMode"))->GetValueOnAnyThread();
	// mode 2 is to use an MSAA target as the swapchain image (depth only for now) and then we copy it's resolved target to the compositor
	if (Mode == 2)
	{
		static int MSAACount = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MSAACount"))->GetValueOnAnyThread();
		return MSAACount;
	}
	return 1;
}

FOXRVisionOSSwapchain::FSwapchainImage::FSwapchainImage(FTextureRHIRef InImage, EImageState InImageState)
	: Image(InImage)
	, ImageState(InImageState)
{

}

FOXRVisionOSSwapchain::FSwapchainImage::~FSwapchainImage() = default;

XrResult FOXRVisionOSSwapchain::Create(TSharedPtr<FOXRVisionOSSwapchain, ESPMode::ThreadSafe>& OutSwapchain, const XrSwapchainCreateInfo* createInfo, uint32 SwapchainLength, FOXRVisionOSSession* Session)
{
	if (createInfo == nullptr || Session == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	if (createInfo->type != XR_TYPE_SWAPCHAIN_CREATE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	OutSwapchain = MakeShared<FOXRVisionOSSwapchain, ESPMode::ThreadSafe>(createInfo, SwapchainLength, Session);
	if (OutSwapchain->CreateFailureResult != XrResult::XR_SUCCESS)
	{
		XrResult Ret = OutSwapchain->CreateFailureResult;
		OutSwapchain = nullptr;
		return Ret;
	}

	return XrResult::XR_SUCCESS;
}

FOXRVisionOSSwapchain::FOXRVisionOSSwapchain(const XrSwapchainCreateInfo* createInfo, uint32 SwapchainLength, FOXRVisionOSSession* InSession)
{
	Session = InSession;

	check(createInfo->next == nullptr);

	check(createInfo);
	CreateInfo = *createInfo;

	{
		check(CreateInfo.faceCount == 1);
		check(CreateInfo.arraySize == 1);
		
		const bool isColor = (CreateInfo.usageFlags & XR_SWAPCHAIN_USAGE_COLOR_ATTACHMENT_BIT) != 0;
		const bool isDepth = (CreateInfo.usageFlags & XR_SWAPCHAIN_USAGE_DEPTH_STENCIL_ATTACHMENT_BIT) != 0;
		check((isColor || isDepth) && (isColor != isDepth))

		ETextureCreateFlags TextureCreateFlags = ETextureCreateFlags::None;
		TextureCreateFlags |= TexCreate_RenderTargetable;
		TextureCreateFlags |= TexCreate_ShaderResource;
		uint8 NumSamples = 1;
		if (isColor)
		{
			TextureCreateFlags |= TexCreate_UAV;
			TextureCreateFlags |= TexCreate_SRGB;
		} 
		else
		{
			NumSamples = GetMSAASampleCount();
			if ((EPixelFormat)CreateInfo.format == PF_DepthStencil)
			{
				TextureCreateFlags |= TexCreate_DepthStencilTargetable;
				TextureCreateFlags |= TexCreate_DepthStencilResolveTarget;
				TextureCreateFlags |= TexCreate_InputAttachmentRead;
			}
			else if ((EPixelFormat)CreateInfo.format == PF_R32_FLOAT)
			{
				TextureCreateFlags |= TexCreate_DepthStencilTargetable;
				TextureCreateFlags |= TexCreate_ResolveTargetable;
				TextureCreateFlags |= TexCreate_InputAttachmentRead;
			}
			else
			{
				check(false);
			}
		}

		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

		const FRHITextureCreateDesc BackBufferDesc =
			FRHITextureCreateDesc::Create2D(isColor ? TEXT("OXRVisionOSSwapchain") : TEXT("OXRVisionOSSwapchainDepth"),
											CreateInfo.width, CreateInfo.height,
											(EPixelFormat)CreateInfo.format)
			.SetFlags(TextureCreateFlags)
			.SetNumSamples(NumSamples)
			.DetermineInititialState();
		
		Images.Reserve(SwapchainLength);
		for (int32 BufferIndex = 0; BufferIndex < SwapchainLength; ++BufferIndex)
		{
			const FString Name = FString::Printf(TEXT("%s_SwapChain%d"), (CreateInfo.usageFlags & XR_SWAPCHAIN_USAGE_COLOR_ATTACHMENT_BIT) ? TEXT("Color") : TEXT("Depth"), BufferIndex);

			FRHITextureCreateDesc CurrentBackBufferDesc = BackBufferDesc;
			CurrentBackBufferDesc.DebugName = *Name;

			FTextureRHIRef Tex = RHICmdList.CreateTexture(CurrentBackBufferDesc);
			Images.Emplace(Tex, EImageState::Released);
		}
	}
}

FOXRVisionOSSwapchain::~FOXRVisionOSSwapchain()
{
	if (CreateFailureResult != XrResult::XR_SUCCESS)
	{
		UE_LOGF(LogOXRVisionOS, Warning, "Destructing FOXRVisionOSSwapchain because create failed.");
	}

	//SwapchainWaitManager.Reset();
}

XrResult FOXRVisionOSSwapchain::XrDestroySwapchain()
{
	// Session->DestroySwapchain() can delete this, so better just return after that.
	return Session->DestroySwapchain(this);
}

XrResult FOXRVisionOSSwapchain::XrEnumerateSwapchainImages(
	uint32_t                                    imageCapacityInput,
	uint32_t*									imageCountOutput,
	XrSwapchainImageBaseHeader*					images)
{
	if (imageCountOutput == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}
	if (imageCapacityInput != 0 && images == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	*imageCountOutput = Images.Num();
	if (imageCapacityInput != 0)
	{
		if (imageCapacityInput < *imageCountOutput)
		{
			return XrResult::XR_ERROR_SIZE_INSUFFICIENT;
		}

		XrSwapchainImageOXRVisionOS* CastImages = (XrSwapchainImageOXRVisionOS*)images;
		for (uint32_t i = 0; i < *imageCountOutput; ++i)
		{
			XrSwapchainImageOXRVisionOS& Output = CastImages[i];
			check(Output.type == (XrStructureType)XR_TYPE_SWAPCHAIN_IMAGE_OXRVISIONOS_EPIC)
			Output.next = nullptr;
			Output.image = Images[i].Image;
		}
	}
	return XrResult::XR_SUCCESS;
}

const FOXRVisionOSSwapchain::FSwapchainImage& FOXRVisionOSSwapchain::GetLastWaitedImage() const
{
	check(NextImageToWait < Images.Num());

	const FSwapchainImage& WaitedImage = Images[NextImageToWait];
	//check(WaitedImage.ImageState == EImageState::WaitComplete);

	UE_LOGF(LogOXRVisionOS, VeryVerbose, "FOXRVisionOSSwapchain::GetLastWaitedImage() NextImageToWait=%i Metal texture = 0x%x", NextImageToWait, WaitedImage.Image.GetReference()->GetNativeResource());
	
	return WaitedImage;
}

const FOXRVisionOSSwapchain::FSwapchainImage& FOXRVisionOSSwapchain::GetLastReleasedImage() const
{
	check(NextImageToRelease < Images.Num());

	// OXR spec says xrEndFrame uses last released image, so we don't really care about the index
	// just whichever was last

	const FSwapchainImage& ReleasedImage = Images[NextImageToRelease];
	check(ReleasedImage.ImageState == EImageState::Released);

	UE_LOGF(LogOXRVisionOS, VeryVerbose, "FOXRVisionOSSwapchain::GetLastReleasedImage() NextImageToRelease=%i Metal texture = 0x%x", NextImageToRelease, ReleasedImage.Image.GetReference()->GetNativeResource());
	
	return ReleasedImage;
}


XrResult FOXRVisionOSSwapchain::XrAcquireSwapchainImage(
	const XrSwapchainImageAcquireInfo*			acquireInfo,
	uint32_t*									index)
{
	SCOPED_NAMED_EVENT_TEXT("FOXRVisionOSSwapchain::XrAcquireSwapchainImage", FColor::Turquoise);

	if (index == nullptr)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	// acquireInfo can be null
	if (acquireInfo != nullptr && acquireInfo->type != XR_TYPE_SWAPCHAIN_IMAGE_ACQUIRE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	FSwapchainImage& NextImage = Images[NextImageToAcquire];
	if (NextImage.ImageState == EImageState::Released)
	{
		NextImage.ImageState = EImageState::Aquired;
		*index = NextImageToAcquire;
		
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "FOXRVisionOSSwapchain::XrAcquireSwapchainImage() Acquired %i metal texture 0x%x", NextImageToAcquire, NextImage.Image.GetReference()->GetNativeResource());

		AcquiredImageQueue.Enqueue(NextImageToAcquire);

		// TODO: Can the swapchain image advance on it's own??
		// probably not because of tightly coupled images + buffers for videoout
		NextImageToAcquire = (NextImageToAcquire + 1) % Images.Num();

		return XrResult::XR_SUCCESS;
	}
	else
	{
		return XrResult::XR_ERROR_CALL_ORDER_INVALID;
	}
}

XrResult FOXRVisionOSSwapchain::XrWaitSwapchainImage(
	const XrSwapchainImageWaitInfo*				waitInfo)
{
	SCOPED_NAMED_EVENT_TEXT("FOXRVisionOSSwapchain::XrWaitSwapchainImage", FColor::Turquoise);

	if (waitInfo == nullptr || waitInfo->type != XR_TYPE_SWAPCHAIN_IMAGE_WAIT_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}


	// TODO
	// We can only wait on one image at a time. Any previously waited on image
	// has to be released! That _should_ be impossible because OnFinishRendering_RHIThread
	// should run again before OnBeginRendering_RHIThread is scheduled again. But it could
	// happen if present isn't called or OnFinish bails early!
	if (++OutstandingWaits > 1)
	{
		// We won't recover because xrReleaseSwapchainImage is scheduled
		// synchronously after xrWaitSwapchainImage :(
		// Is there a better return code for this?
		return XrResult::XR_TIMEOUT_EXPIRED;
	}

	if (!AcquiredImageQueue.Dequeue(NextImageToWait))
	{
		return XrResult::XR_ERROR_CALL_ORDER_INVALID;
	}

	FSwapchainImage& NextImage = Images[NextImageToWait];
	if (NextImage.ImageState == EImageState::Aquired)
	{
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "FOXRVisionOSSwapchain::XrWaitSwapchainImage() Waiting on NextImageToWait %i ", NextImageToWait);
		NextImage.ImageState = EImageState::Waiting;

		//custom image ready wait could be here, if there was one.

		NextImage.ImageState = EImageState::WaitComplete;
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "FOXRVisionOSSwapchain::XrWaitSwapchainImage() WaitComplete on NextImageToWait %i  metal texture 0x%x", NextImageToWait, NextImage.Image.GetReference()->GetNativeResource());
		return XrResult::XR_SUCCESS;
	}
	else
	{
		return XrResult::XR_ERROR_CALL_ORDER_INVALID;
	}
}

XrResult FOXRVisionOSSwapchain::XrReleaseSwapchainImage(
	const XrSwapchainImageReleaseInfo*			releaseInfo)
{
	SCOPED_NAMED_EVENT_TEXT("FOXRVisionOSSwapchain::XrReleaseSwapchainImage", FColor::Turquoise);

	// releaseInfo can be null
	if (releaseInfo != nullptr && releaseInfo->type != XR_TYPE_SWAPCHAIN_IMAGE_RELEASE_INFO)
	{
		return XrResult::XR_ERROR_VALIDATION_FAILURE;
	}

	// OXR spec says only one image can be waited on, therefore,
	// the last used wait index is the outstanding wait
	NextImageToRelease = NextImageToWait;

	FSwapchainImage& NextImage = Images[NextImageToRelease];
	if (NextImage.ImageState == EImageState::WaitComplete)
	{
		UE_LOGF(LogOXRVisionOS, VeryVerbose, "FOXRVisionOSSwapchain::XrReleaseSwapchainImage() Releasing %i metal texture 0x%x", NextImageToRelease, NextImage.Image.GetReference()->GetNativeResource());
		
		NextImage.ImageState = EImageState::Released;
		OutstandingWaits--;
		return XrResult::XR_SUCCESS;
	}
	else
	{
		return XrResult::XR_ERROR_CALL_ORDER_INVALID;
	}
}
