// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OXRVisionOS.h"

#include "Runtime/Launch/Resources/Version.h"
#include "Templates/RefCounting.h"

class FOXRVisionOSSession;
namespace MetalRHIOXRVisionOS
{
	class ISwapchainLabelManager;
}


class FOXRVisionOSSwapchain
{
public:
	enum class EImageState
	{
		Released,
		Aquired,
		Waiting,
		WaitComplete
	};

	struct FSwapchainImage
	{
		FSwapchainImage(FTextureRHIRef InImage, EImageState InImageState);
		~FSwapchainImage();

		FTextureRHIRef Image;
		EImageState ImageState = EImageState::Released;
	};

	static XrResult Create(TSharedPtr<FOXRVisionOSSwapchain, ESPMode::ThreadSafe>& OutSwapchain, const XrSwapchainCreateInfo* createInfo, uint32 SwapchainLength, FOXRVisionOSSession* Session);

	FOXRVisionOSSwapchain(const XrSwapchainCreateInfo* createInfo, uint32 SwapchainLength, FOXRVisionOSSession* Session);
	~FOXRVisionOSSwapchain();
	XrResult XrDestroySwapchain();

	XrResult XrEnumerateSwapchainImages(
		uint32_t                                    imageCapacityInput,
		uint32_t*									imageCountOutput,
		XrSwapchainImageBaseHeader*					images);

	XrResult XrAcquireSwapchainImage(
		const XrSwapchainImageAcquireInfo*			acquireInfo,
		uint32_t*									index);

	XrResult XrWaitSwapchainImage(
		const XrSwapchainImageWaitInfo*				waitInfo);

	XrResult XrReleaseSwapchainImage(
		const XrSwapchainImageReleaseInfo*			releaseInfo);
	
	const FSwapchainImage& GetLastWaitedImage() const;

	const FSwapchainImage& GetLastReleasedImage() const;

private:
	//bool WaitLabel_RHIThread(uint32 AcquireImageIndex);
	void ReleaseLabel_RHIThread(uint32 ReleaseImageIndex, uint64 LabelValue);
	
	XrResult CreateFailureResult = XrResult::XR_SUCCESS;
	FOXRVisionOSSession*		Session = nullptr;

	XrSwapchainCreateInfo CreateInfo;

	TArray<FSwapchainImage> Images;
	uint32_t NextImageToAcquire = 0;
	uint32_t NextImageToWait = 0;
	uint32_t NextImageToRelease = 0;


	TQueue<uint32_t> AcquiredImageQueue;
	uint32_t OutstandingWaits = 0;
};
