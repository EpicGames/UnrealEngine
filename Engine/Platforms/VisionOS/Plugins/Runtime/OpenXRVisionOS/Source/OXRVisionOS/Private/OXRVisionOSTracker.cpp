// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSTracker.h"

DECLARE_LOG_CATEGORY_CLASS(LogOXRVisionOSTracker, Log, All);

/** includes */
#include "OXRVisionOSPlatformUtils.h"
#include "OXRVisionOSController.h"
#include "OXRVisionOSInstance.h"
#include "OXRVisionOSSession.h"

bool FOXRVisionOSTracker::Create(TSharedPtr<FOXRVisionOSTracker, ESPMode::ThreadSafe>& OutTracker, bool bEnableGaze)
{
	OutTracker = MakeShared<FOXRVisionOSTracker, ESPMode::ThreadSafe>(bEnableGaze);
	if (OutTracker->bCreateFailed)
	{
		OutTracker = nullptr;
		return false;
	}
	return true;
}

//=============================================================================
FOXRVisionOSTracker::FOXRVisionOSTracker(bool bEnableGaze)// :
//	bGazeEnabled(bEnableGaze)
{

	//if (Ret != OK)
	//{
	//	UE_LOGF(LogOXRVisionOSTracker, Warning, "Initialize failed: 0x%08X", Ret);
	//	bCreateFailed = true;
	//	return;
	//}
}

//=============================================================================
FOXRVisionOSTracker::~FOXRVisionOSTracker()
{
	// if (Result != OK)
	// {
	// 	UE_LOGF(LogOXRVisionOSTracker, Warning, "Term failed: 0x%08X", Result);
	// }
}


bool FOXRVisionOSTracker::RegisterDevice(TrackerDeviceType DeviceType, int32 DeviceHandle)
{
	FScopeLock ScopeLock(&DeviceInfoMapCriticalSection);

	FDeviceInfo& Added = DeviceInfoMap.Add(DeviceType, FDeviceInfo(DeviceHandle));

	TryRegisterDevice(DeviceType, Added);

	return true;
}

bool FOXRVisionOSTracker::UnregisterDevice(int32 DeviceHandle)
{
	FScopeLock ScopeLock(&DeviceInfoMapCriticalSection);

	for (auto& Elem : DeviceInfoMap)
	{
		if (DeviceHandle == Elem.Value.Handle)
		{
			DeviceInfoMap.Remove(Elem.Key);
			break;
		}
	}
	return true;
}

#define OXRVISIONOS_TRACKER_RESULT_LOGGING 0

void FOXRVisionOSTracker::SyncActions(int32 FrameCounter, XrTime PredictedDisplayTime)
{
	FScopeLock ScopeLock(&DeviceInfoMapCriticalSection);

	// // Gaze tracker result caching
	// if (bGazeEnabled)
	// {
	// 	// Null device info means we have not registered the hmd for tracking yet, so we can't get a result
	// 	if (const FDeviceInfo* DeviceInfo = DeviceInfoMap.Find(DEVICE_HMD))
	// 	{
	// 		GazeResults.LastResultIndex = (GazeResults.LastResultIndex + 1) % FGazeResults::MaxResultBufferSize;
	// 		GazeResults.NumResultsBuffered = FMath::Max(GazeResults.NumResultsBuffered, GazeResults.LastResultIndex + 1);
	// 		GazeResult& Result = GazeResults.ResultBuffer[GazeResults.LastResultIndex];

	// 		GazeGetResult(&Result);
	// 		if (Result.Failed)
	// 		{
	// 			UE_LOGF(LogOXRVisionOSTracker, Warning, "SyncActions GazeGetResult failed");
	// 		}
	// 	}
	// }
}

void FOXRVisionOSTracker::TryRegisterDevice(TrackerDeviceType DeviceType, FDeviceInfo& Info)
{
	// int32 Result = APIRegisterDevice(DeviceType, Info.Handle);
	// if (Result != RESULT_OK)
	// {
	// 	UE_LOGF(LogOXRVisionOSTracker, Warning, "RegisterDevice failed code 0x%x. \n", Result);
	// }
	// else
	// {
	// 	Info.bIsActive = true;
	// }

	if (DeviceType == TrackerDeviceType::DEVICE_VR_HMD)
	{
		//TODO
		Info.Handle = 0;
		Info.bIsActive = true;
	}
	
}

bool FOXRVisionOSTracker::GetResult(int32 DeviceHandle, XrTime DisplayTime, OXRVisionOS::TrackerResultData& ResultData)
{
	SCOPED_NAMED_EVENT_TEXT("FOXRVisionOSTracker::GetResult", FColor::Turquoise);

	FScopeLock ScopeLock(&DeviceInfoMapCriticalSection);

	bool bFound = false;
	TrackerDeviceType DeviceType;
	FDeviceInfo* DeviceInfo = nullptr;
	for (auto& Elem : DeviceInfoMap)
	{
		if (DeviceHandle == Elem.Value.Handle)
		{
			if (Elem.Value.bIsActive == false)
			{
				// Device not yet active.
				return false;
			}
			else
			{
				DeviceInfo = &Elem.Value;
				DeviceType = Elem.Key;
				bFound = true;
				break;
			}
		}
	}

	if (bFound == false)
	{
		UE_LOGF(LogOXRVisionOSTracker, Warning, "FOXRVisionOSTracker::GetResult called with unrecognized DeviceHandle %i", DeviceHandle);
		return false;
	}

	//TODO: get the pose data
	//{
	//	SCOPED_NAMED_EVENT_TEXT("GetResult", FColor::Turquoise);
	//	ResultData.Pose =
	//}

// #if OXRVISIONOS_TRACKER_RESULT_LOGGING
// 	LogGetResult(DeviceType, Result, ResultData);
// #endif

// 	if (Result != RESULT_OK)
// 	{
// 		UE_LOGF(LogOXRVisionOSTracker, Warning, "GetResult failed: %d (%u, %08X)", Result, Result, Result);
// 		return false;
// 	}

	return true;
}

// bool FOXRVisionOSTracker::GetGazeResult(int32 DeviceHandle, XrTime DisplayTime, int32 FrameNumber, GazeResult& OutGazeResult, ResultData& OutHmdResult)
// {
// 	if (bGazeEnabled == false)
// 	{
// 		UE_LOGF(LogOXRVisionOSTracker, Warning, "FOXRVisionOSTracker::GetGazeResult called without gaze tracking enabled");
// 		return false;
// 	}

// 	OutGazeResult.GazeDirection.x = 0.0f;
// 	OutGazeResult.GazeDirection.y = 0.0f;
// 	OutGazeResult.GazeDirection.z = 1.0f;

// 	// Search GazeResults for the result closest to DisplayTime.
// 	// https://www.khronos.org/registry/OpenXR/specs/1.0/html/xrspec.html#_eye_gaze_input
// 	// One particularity for eye trackers compared to most other spatial input is that the runtime may not have the capability to predict or interpolate eye gaze poses.
// 	// Runtimes that cannot predict or interpolate eye gaze poses must clamp the gaze pose requested in the xrLocateSpace call to the value nearest to time requested in the call.
// 	// We will search starting with the most recent.
// 	GazeResult* FoundGazeResult = nullptr;
// 	uint64_t TargetTime = OXRVisionOS::XrTimeToApiTime(DisplayTime);
// 	uint64_t TimeDiff = UINT64_MAX;
// 	if (GazeResults.NumResultsBuffered == 0)
// 	{
// 		return false; // no results yet
// 	}
// 	const int NumResults = GazeResults.NumResultsBuffered;
// 	for (int i = 0; i < NumResults; ++i)
// 	{
// 		int Index = (GazeResults.LastResultIndex - i + NumResults) % NumResults;
// 		GazeResult& Result = GazeResults.ResultBuffer[Index];

// 		uint64_t NewTimeDiff;
// 		if (Result.timestamp == TargetTime)  // this may seem unlikely, but the timestamps may come from a common source.
// 		{
// 			FoundGazeResult = &Result;
// 			break;
// 		}
// 		else if (TargetTime > Result.timestamp) // result older
// 		{
// 			NewTimeDiff = TargetTime - Result.timestamp;
// 		}
// 		else // result newer
// 		{
// 			NewTimeDiff = Result.timestamp - TargetTime;
// 		}

// 		if (NewTimeDiff < TimeDiff)
// 		{
// 			// This one is better.
// 			FoundGazeResult = &Result;
// 			TimeDiff = NewTimeDiff;
// 		}
// 		else
// 		{
// 			// This one is worse, so the last one was best.
// 			break;
// 		}
// 	}

// 	check(FoundGazeResult);
// 	OutGazeResult = *FoundGazeResult;

// 	bool bIsGazeDataAvailable = false;
// 	if (OutGazeResult.status == ENABLED)
// 	{
// 		if (OutGazeResult.isGazeValid)
// 		{
// 			// gaze data is up to date
// 			bIsGazeDataAvailable = true;
// 		}
// 	}
// 	else
// 	{
// 		bIsGazeDataAvailable = false;
// 	}

// 	if (!bIsGazeDataAvailable)
// 	{
// 		return false;
// 	}

// 	// Potentially transform gaze data into tracking space, if it isn't already.

// 	if (Result != RESULT_OK)
// 	{
// 		UE_LOGF(LogOXRVisionOSTracker, Warning, "GetResult failed: %d (%u, %08X)", Result, Result, Result);
// 		return false;
// 	}

// 	return true;
// }

bool FOXRVisionOSTracker::UpdateReferenceSpaces(class FOXRVisionOSInstance* Instance, FOXRVisionOSSession* Session, int32 DeviceHandle, XrTime DisplayTime, int32 FrameNumber)
{
	OXRVisionOS::TrackerResultData ResultData = {};
	bool bSuccess = GetResult(DeviceHandle, DisplayTime, ResultData);
	if (!bSuccess)
	{
		return false;
	}

	// if (ResultData.playAreaUpdateCounter == PlayAreaData.PlayAreaUpdateCounter)
	// {
	// 	return false;
	// }


	// UE_LOGF(LogOXRVisionOSTracker, Log, "FOXRVisionOSTracker::UpdateReferenceSpaces update happening. %i", ResultData.playAreaUpdateCounter);
	// PlayAreaData.PlayAreaUpdateCounter = ResultData.playAreaUpdateCounter;

	// if (PlayAreaData.PlayAreaMode != ResultData.playAreaMode)
	// {
	// 	UE_LOGF(LogOXRVisionOSTracker, Log, "FOXRVisionOSTracker::UpdateReferenceSpaces playAreaMode changing from %i to %i", PlayAreaData.PlayAreaMode, ResultData.playAreaMode);
	// 	PlayAreaData.PlayAreaMode = ResultData.playAreaMode;
	// }

//	{
//		check(Instance);
//		check(Session);
//
//		// Send the bounds updated event!
//		XrEventDataReferenceSpaceChangePending Event;
//		Event.type = XR_TYPE_EVENT_DATA_REFERENCE_SPACE_CHANGE_PENDING;
//		Event.next = nullptr;
//		Event.session = (XrSession)this;
//		Event.referenceSpaceType = XR_REFERENCE_SPACE_TYPE_STAGE;
//		Event.changeTime = Session->GetCurrentTime();
//		Event.poseValid = false;//PlayAreaData.PlayAreaOrientedBoundingBoxDataIsValid;  // Currently we are not trying to provide the pose change update here, the spec allows this.  See XrEventDataReferenceSpaceChangePending in the OpenXR reference pages.
//		//Event.poseInPreviousSpace = ;
//
//		Instance->EnqueueEvent(Event);
//	}

	return true;
}

bool FOXRVisionOSTracker::GetStagePose(OXRVisionOS::TrackerPoseData& OutPoseData) const
{
	// OpenXR requires us to enumerate spaces on startup.
	// OpenXRHMD assumes that if it can enumerate a stage space that will work.  I'm not sure OpenXR actually guarantees that.
	// It talks about a stage space unable to locate...
	// int32_t Result = LocateCoordinate(STAGE, &OutPoseData);
	// return (Result == RESULT_OK);
	return true;
}

void FOXRVisionOSTracker::GetActionStatePose(EOXRVisionOSControllerButton Button, bool& OutActive)
{
	// Merge rule: doesn't change
	// We are only returning active here, one gets the pose via xrCreateActionSpace and xrLocateSpace.

	FScopeLock ScopeLock(&DeviceInfoMapCriticalSection);

	switch (Button)
	{
	case EOXRVisionOSControllerButton::GripL:
	case EOXRVisionOSControllerButton::AimL:
	case EOXRVisionOSControllerButton::GripSurfaceL:
	{
		FDeviceInfo* DeviceInfo = DeviceInfoMap.Find(TrackerDeviceType::DEVICE_VR_CONTROLLER_LEFT);
		OutActive = DeviceInfo->bIsActive; //TODO is this correct?
		return; 
	}
	case EOXRVisionOSControllerButton::GripR:
	case EOXRVisionOSControllerButton::AimR:
	case EOXRVisionOSControllerButton::GripSurfaceR:
	{
		FDeviceInfo* DeviceInfo = DeviceInfoMap.Find(TrackerDeviceType::DEVICE_VR_CONTROLLER_RIGHT);
		OutActive = DeviceInfo->bIsActive; //TODO is this correct?
		return;
	}
	// case EOXRVisionOSControllerButton::GazePose:
	// {
	// 	OutActive = GazeResults.NumResultsBuffered > 0 && (GazeResults.ResultBuffer[GazeResults.LastResultIndex].status == GAZE_TRACKING_STATUS_ENABLE);
	// 	return;
	// }
	default:
		check(false);
		OutActive = false;
		return;
	}
}
