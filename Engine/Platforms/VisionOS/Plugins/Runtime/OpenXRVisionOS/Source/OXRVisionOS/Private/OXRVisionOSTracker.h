// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OpenXRCore.h"
#include "Misc/ScopeLock.h"

enum class EOXRVisionOSControllerButton : int32;
class FOXRVisionOSInstance;
class FOXRVisionOSSession;

namespace OXRVisionOS
{
	struct TrackerResultData;
	struct TrackerPoseData;
}

enum class TrackerDeviceType : int32
{
	DEVICE_VR_HMD,
	DEVICE_VR_CONTROLLER_LEFT,
	DEVICE_VR_CONTROLLER_RIGHT
};

class FOXRVisionOSTracker
{
public:
	static bool Create(TSharedPtr<FOXRVisionOSTracker, ESPMode::ThreadSafe>& OutTracker, bool bEnableGaze);
	FOXRVisionOSTracker(bool bEnableGaze);
	~FOXRVisionOSTracker();

	bool RegisterDevice(TrackerDeviceType DeviceType, int32 DeviceHandle);
	bool UnregisterDevice(int32 DeviceHandle);

	bool GetResult(int32 DeviceHandle, XrTime DisplayTime, OXRVisionOS::TrackerResultData& OutResultData);
//	bool GetGazeResult(int32 DeviceHandle, XrTime DisplayTime, GazeResult& OutResult, ResultData& OutHmdResult);
	bool UpdateReferenceSpaces(FOXRVisionOSInstance* Instance, FOXRVisionOSSession* Session, int32 DeviceHandle, XrTime DisplayTime, int32 FrameNumber);
	bool GetStagePose(OXRVisionOS::TrackerPoseData& OutPoseData) const; // Get the pose of the stage space in device tracking space.

	void GetActionStatePose(EOXRVisionOSControllerButton Button, bool& OutActive);

	void SyncActions(int32 FrameCounter, XrTime PredictedDisplayTime);

private:
	const int32 RESULT_OK = 0; //TEMP

	bool bCreateFailed = false;
	void* TrackingRam = nullptr;

	struct FDeviceInfo
	{
		explicit FDeviceInfo(int32 InHandle)
			: Handle(InHandle)
		{}

		int32 Handle = -1;
		bool bIsActive = false;
	};

	// bool bGazeEnabled;
	// // We want to be able to get historical results 50ms in the past.  7 120fps frames is 56ms.
	// struct FGazeResults
	// {
	// 	static const int32 MaxResultBufferSize = 7;
	// 	int32 LastResultIndex = -1;
	// 	int32 NumResultsBuffered = 0;
	// 	dddGazeResult ResultBuffer[MaxResultBufferSize] = {};
	// } GazeResults;


	TMap< TrackerDeviceType, FDeviceInfo> DeviceInfoMap;
	mutable FCriticalSection DeviceInfoMapCriticalSection;
	void TryRegisterDevice(TrackerDeviceType DeviceType, FDeviceInfo& Info);
};
