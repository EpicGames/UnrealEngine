// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOS_RenderBridge.h"

#include "OXRVisionOS_Swapchain.h"
#include "OpenXRCore.h"
#include "OXRVisionOS_openxr_platform.h"
#include "OXRVisionOSPlatformRHI.h"

#include "OXRVisionOSInstance.h"
#include "OXRVisionOSSession.h"
#include "RHICommandList.h"

class FOXRVisionOSRenderBridge : public FOpenXRRenderBridge
{
public:
	FOXRVisionOSRenderBridge(XrInstance InInstance)
		: FOpenXRRenderBridge(InInstance)
		, Binding()
	{
	}

	virtual void* GetGraphicsBinding(XrSystemId InSystem) override
	{
		Binding.type = (XrStructureType)XR_TYPE_GRAPHICS_BINDING_OXRVISIONOS_EPIC;
		Binding.next = nullptr;
		return &Binding;
	}

	virtual FXRSwapChainPtr CreateSwapchain(XrSession InSession, uint8 Format, uint8& OutFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding, ETextureCreateFlags AuxiliaryCreateFlags) override final
	{
		return CreateSwapchain_OXRVisionOS(InSession, Format, OutFormat, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags, ClearValueBinding);
	}

	//virtual bool Support10BitSwapchain() const override { return true; }

	virtual bool HDRGetMetaDataForStereo(EDisplayOutputFormat& OutDisplayOutputFormat, EDisplayColorGamut& OutDisplayColorGamut, bool& OutbHDRSupported) override
	{
		OutDisplayOutputFormat = EDisplayOutputFormat::SDR_ExplicitGammaMapping;
		OutDisplayColorGamut = EDisplayColorGamut::DCIP3_D65;
		OutbHDRSupported = true;
		return true;
	}

private:

	XrGraphicsBindingOXRVisionOSEPIC Binding;
};

FOpenXRRenderBridge* CreateRenderBridge_OXRVisionOS(XrInstance InInstance) { return new FOXRVisionOSRenderBridge(InInstance); }
