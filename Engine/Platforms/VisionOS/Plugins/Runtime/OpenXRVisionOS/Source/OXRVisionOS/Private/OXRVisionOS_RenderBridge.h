// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenXRHMD_RenderBridge.h"

FOpenXRRenderBridge* CreateRenderBridge_OXRVisionOS(XrInstance InInstance);

