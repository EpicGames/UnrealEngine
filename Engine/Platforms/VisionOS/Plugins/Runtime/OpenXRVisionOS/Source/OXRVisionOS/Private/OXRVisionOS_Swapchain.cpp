// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOS_Swapchain.h"
#include "OpenXRCore.h"
#include "OpenXRPlatformRHI.h"
#include "OpenXRHMD_Swapchain.h"
#include "OXRVisionOS_openxr_platform.h"
#include "OXRVisionOSPlatformRHI.h"

//UE_TRACE_CHANNEL_EXTERN(OpenXRChannel)

static TArray<XrSwapchainImageOXRVisionOS> EnumerateImagesOXRVisionOS(XrSwapchain InSwapchain, XrStructureType InType)
{
	TArray<XrSwapchainImageOXRVisionOS> Images;
	uint32_t ChainCount;
	xrEnumerateSwapchainImages(InSwapchain, 0, &ChainCount, nullptr);
	Images.AddZeroed(ChainCount);
	for (auto& Image : Images)
	{
		Image.type = InType;
	}
	XR_ENSURE(xrEnumerateSwapchainImages(InSwapchain, ChainCount, &ChainCount, reinterpret_cast<XrSwapchainImageBaseHeader*>(Images.GetData())));
	return Images;
}

FXRSwapChainPtr CreateSwapchain_OXRVisionOS(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding)
{
	if (Dimension != ETextureDimension::Texture2D)
	{
		UE_LOGF(LogHMD, Error, "Unsupported texture dimension for VisionOS");
		ensure(false);
		return nullptr;
	}

	// We use UE formats in OXRVisionOS
	TFunction<uint32(uint8)> ToPlatformFormat = [](uint8 InFormat) { return InFormat; };

	Format = FOpenXRSwapchain::GetNearestSupportedSwapchainFormat(InSession, Format, ToPlatformFormat);
	if (!Format)
	{
		return nullptr;
	}

	OutActualFormat = Format;
	XrSwapchain Swapchain = FOpenXRSwapchain::CreateSwapchain(InSession, Format, SizeX, SizeY, ArraySize, NumMips, NumSamples, Dimension, CreateFlags);
	if (!Swapchain)
	{
		return nullptr;
	}

	TArray<FTextureRHIRef> TextureChain;
	TArray<XrSwapchainImageOXRVisionOS> Images = EnumerateImagesOXRVisionOS(Swapchain, (XrStructureType)XR_TYPE_SWAPCHAIN_IMAGE_OXRVISIONOS_EPIC);
	for (const auto& Image : Images)
	{
		TextureChain.Add(Image.image);
	}
	//FTextureRHIRef ChainTarget;//= static_cast<FTextureRHIRef>(GDynamicRHI->RHICreateAliasedTexture((FTextureRHIRef&)TextureChain[0]));
	FTextureRHIRef ChainTarget = (FTextureRHIRef&)TextureChain[0];

	return CreateXRSwapChain<FOpenXRSwapchain>(MoveTemp(TextureChain), ChainTarget, Swapchain);
}
