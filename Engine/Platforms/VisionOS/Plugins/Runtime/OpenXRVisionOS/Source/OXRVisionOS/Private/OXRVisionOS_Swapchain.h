// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XRSwapChain.h"
#include "OpenXRPlatformRHI.h"
#include "Runtime/Launch/Resources/Version.h"

#include <openxr/openxr.h>

FXRSwapChainPtr CreateSwapchain_OXRVisionOS(XrSession InSession, uint8 Format, uint8& OutActualFormat, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, ETextureDimension Dimension, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding);
