// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("VisionOS", "Win64", "Mac")]
public class OXRVisionOSSettings : ModuleRules
{
	public OXRVisionOSSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "Core",
                "CoreUObject",
                "Engine",
			}
		);

		PublicDependencyModuleNames.Add("DeveloperSettings");

		// Needed to include ISettingsModule.h
		PrivateIncludePathModuleNames.Add("Settings");
	}
}
