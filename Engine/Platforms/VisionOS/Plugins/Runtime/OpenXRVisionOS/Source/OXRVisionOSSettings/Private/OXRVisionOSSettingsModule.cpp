// Copyright Epic Games, Inc. All Rights Reserved.

#include "OXRVisionOSSettingsModule.h"
#include "OXRVisionOSRuntimeSettings.h"
#include "Modules/ModuleManager.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Class.h"

// Settings
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "OXRVisionOSSettings"

//////////////////////////////////////////////////////////////////////////
// FOXRVisionOSSettings

class FOXRVisionOSSettings : public IOXRVisionOSSettingsModule
{
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FOXRVisionOSSettings, OXRVisionOSSettings);

//Note: this one is required to get the setting to show up in Editor->Project Settings->Plugins
FName UOXRVisionOSRuntimeSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UOXRVisionOSRuntimeSettings::GetSectionText() const
{
	return NSLOCTEXT("OXRVisionOSSettings", "OpenXRVisionOSRuntimeSettingsSection", "OpenXR visionOS");
}

FText UOXRVisionOSRuntimeSettings::GetSectionDescription() const
{
	return NSLOCTEXT("OXRVisionOSSettings", "OpenXRVisionOSRuntimeSettingsSectionDescription", "OpenXR visionOS Settings");
}

#endif
//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
