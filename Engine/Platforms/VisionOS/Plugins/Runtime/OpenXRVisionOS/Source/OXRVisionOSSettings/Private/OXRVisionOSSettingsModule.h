// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define OXRVISIONOS_SETTINGS_MODULE_NAME "OXRVisionOSSettings"

//////////////////////////////////////////////////////////////////////////
// IOXRVisionOSSettingsModule

class IOXRVisionOSSettingsModule : public IModuleInterface
{
public:
};
