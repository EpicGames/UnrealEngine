// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "OXRVisionOSSettingsTypes.h"

#include "OXRVisionOSRuntimeSettings.generated.h"

UCLASS(config=Engine, defaultconfig)
class OXRVISIONOSSETTINGS_API UOXRVisionOSRuntimeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FText GetSectionDescription() const override;
#endif
	//~ End UDeveloperSettings interface
	
	// Defines a rotation adjustment to the aim pose from the OpenXR visionOS native controller orientation.  This pose may be fine without adjustment.
	UPROPERTY(config, EditAnywhere, Category="OpenXR visionOS", Meta = (DisplayName = "Aim Pose Adjustment"))
	FRotator OXRVisionOSAimPoseAdjustment = FRotator(0.0f, 0.0f, 0.0f);

	// Defines a rotation adjustment to the grip pose from the OpenXR visionOS native controller orientation.  We provide a default adjustment to grip.
	UPROPERTY(config, EditAnywhere, Category = "OpenXR visionOS", Meta = (DisplayName = "Grip Pose Adjustment"))
	FRotator OXRVisionOSGripPoseAdjustment = FRotator(45.0f,0.0f,0.0f);

	// Defines a rotation adjustment to the grip surface pose from the OpenXR visionOS native controller orientation.  We provide a default adjustment to grip surface.
	UPROPERTY(config, EditAnywhere, Category = "OpenXR visionOS", Meta = (DisplayName = "Grip Surface Pose Adjustment"))
	FRotator OXRVisionOSGripSurfacePoseAdjustment = FRotator(0.0f, 0.0f, 0.0f);

	// Number of UI2D/quad layers supported by OpenXR visionOS, required at app startup time
	UPROPERTY(config, EditAnywhere, Category = "OpenXR visionOS", Meta = (DeprecatedProperty, ClampMin = "0", ClampMax = "4", DisplayName = "Max Number UI Layers"))
	int32 OXRVisionOSNumUILayers = 0;
};
