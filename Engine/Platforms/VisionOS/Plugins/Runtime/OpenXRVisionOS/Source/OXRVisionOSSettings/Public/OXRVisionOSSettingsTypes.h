// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

//#include "OXRVisionOSSettingsTypes.generated.h"

//UENUM()
//enum class EFoo : int8
//{
//	Bar = 0 UMETA(DisplayName = "Baaar"),
//};
